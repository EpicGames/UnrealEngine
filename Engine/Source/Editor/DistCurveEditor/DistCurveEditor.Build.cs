// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DistCurveEditor : ModuleRules
{
	public DistCurveEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("UnrealEd");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "AppFramework",
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"InputCore",
				"Slate",
				"SlateCore",
				"LevelEditor",
				"EditorFramework",
				"UnrealEd"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"PropertyEditor"
			}
		);
	}
}
