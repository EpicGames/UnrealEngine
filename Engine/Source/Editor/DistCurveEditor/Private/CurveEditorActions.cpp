// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorActions.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "CurveEditorCommands"

void FDistCurveEditorCommands::RegisterCommands()
{
	UI_COMMAND(RemoveCurve, "Remove Curve", "Remove Curve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveAllCurves, "Remove All Curves", "Remove All Curves", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetTime, "Set Time", "Set Time", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetValue, "Set Value", "Set Value", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetColor, "Set Color", "Set Color", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteKeys, "Delete Key", "Delete Key", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ScaleTimes, "Scale All Times", "Scales the times of all points of all visible tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ScaleValues, "Scale All Values", "Scales the values of all points of all visible tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ScaleSingleCurveTimes, "Scale Curve Times", "Scales the times of all points of this curve & its sub-curves", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ScaleSingleCurveValues, "Scale Curve Values", "Scales the values of all points of this curve & its sub-curves", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ScaleSingleSubCurveValues, "Scale Sub-Curve Values", "Scales the values of all points of this curve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FitHorizontally, "Horizontal", "Fit Horizontally", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FitVertically, "Vertical", "Fit Vertically", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Fit, "Fit", "Fit to Selected", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PanMode, "Pan", "Pan Mode", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ZoomMode, "Zoom", "Zoom Mode", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CurveAuto, "Auto", "Curve Auto", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CurveAutoClamped, "Auto/Clamped", "Curve Auto/Clamped", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CurveUser, "User", "Curve User", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CurveBreak, "Break", "Curve Break", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(Linear, "Linear", "Linear", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(Constant, "Constant", "Constant", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FlattenTangents, "Flatten", "Flatten Tangents", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StraightenTangents, "Straighten", "Straighten Tangents", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowAllTangents, "Show All", "ShowAll Tangents", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CreateTab, "Create", "Create Tab", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteTab, "Delete", "Delete Tab", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
