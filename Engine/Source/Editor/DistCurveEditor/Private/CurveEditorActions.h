// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/*-----------------------------------------------------------------------------
   FDistCurveEditorCommands
-----------------------------------------------------------------------------*/

class FDistCurveEditorCommands : public TCommands<FDistCurveEditorCommands>
{
public:
	/** Constructor */
	FDistCurveEditorCommands() 
		: TCommands<FDistCurveEditorCommands>("CurveEditor", NSLOCTEXT("Contexts", "CurveEditor", "CurveEditor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	/** See tooltips in cpp for documentation */
	
	TSharedPtr<FUICommandInfo> RemoveCurve;
	TSharedPtr<FUICommandInfo> RemoveAllCurves;
	TSharedPtr<FUICommandInfo> SetTime;
	TSharedPtr<FUICommandInfo> SetValue;
	TSharedPtr<FUICommandInfo> SetColor;
	TSharedPtr<FUICommandInfo> DeleteKeys;
	TSharedPtr<FUICommandInfo> ScaleTimes;
	TSharedPtr<FUICommandInfo> ScaleValues;
	TSharedPtr<FUICommandInfo> ScaleSingleCurveTimes;
	TSharedPtr<FUICommandInfo> ScaleSingleCurveValues;
	TSharedPtr<FUICommandInfo> ScaleSingleSubCurveValues;
	TSharedPtr<FUICommandInfo> FitHorizontally;
	TSharedPtr<FUICommandInfo> FitVertically;
	TSharedPtr<FUICommandInfo> Fit;
	TSharedPtr<FUICommandInfo> PanMode;
	TSharedPtr<FUICommandInfo> ZoomMode;
	TSharedPtr<FUICommandInfo> CurveAuto;
	TSharedPtr<FUICommandInfo> CurveAutoClamped;
	TSharedPtr<FUICommandInfo> CurveUser;
	TSharedPtr<FUICommandInfo> CurveBreak;
	TSharedPtr<FUICommandInfo> Linear;
	TSharedPtr<FUICommandInfo> Constant;
	TSharedPtr<FUICommandInfo> FlattenTangents;
	TSharedPtr<FUICommandInfo> StraightenTangents;
	TSharedPtr<FUICommandInfo> ShowAllTangents;
	TSharedPtr<FUICommandInfo> CreateTab;
	TSharedPtr<FUICommandInfo> DeleteTab;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
