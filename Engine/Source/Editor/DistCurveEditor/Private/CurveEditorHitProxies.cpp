// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorHitProxies.h"

IMPLEMENT_HIT_PROXY(HCurveEditorLabelProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCurveEditorHideCurveProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCurveEditorKeyProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCurveEditorKeyHandleProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCurveEditorLineProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCurveEditorLabelBkgProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCurveEditorHideSubCurveProxy,HHitProxy);
