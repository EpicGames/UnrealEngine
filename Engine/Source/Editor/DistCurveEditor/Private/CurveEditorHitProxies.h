// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HitProxies.h"

/*-----------------------------------------------------------------------------
   Hit Proxies
-----------------------------------------------------------------------------*/

struct HCurveEditorLabelProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32 CurveIndex;

	HCurveEditorLabelProxy(int32 InCurveIndex) :
	HHitProxy(HPP_UI),
		CurveIndex(InCurveIndex)
	{}
};

struct HCurveEditorHideCurveProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32 CurveIndex;

	HCurveEditorHideCurveProxy(int32 InCurveIndex) :
	HHitProxy(HPP_UI),
		CurveIndex(InCurveIndex)
	{}
};

struct HCurveEditorKeyProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32 CurveIndex;
	int32 SubIndex;
	int32 KeyIndex;

	HCurveEditorKeyProxy(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex) :
	HHitProxy(HPP_UI),
		CurveIndex(InCurveIndex),
		SubIndex(InSubIndex),
		KeyIndex(InKeyIndex)
	{}
};

struct HCurveEditorKeyHandleProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32 CurveIndex;
	int32 SubIndex;
	int32 KeyIndex;
	bool bArriving;

	HCurveEditorKeyHandleProxy(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex, bool bInArriving) :
	HHitProxy(HPP_UI),
		CurveIndex(InCurveIndex),
		SubIndex(InSubIndex),
		KeyIndex(InKeyIndex),
		bArriving(bInArriving)
	{}
};

struct HCurveEditorLineProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32 CurveIndex;
	int32 SubIndex;

	HCurveEditorLineProxy(int32 InCurveIndex, int32 InSubIndex) :
	HHitProxy(HPP_UI),
		CurveIndex(InCurveIndex),
		SubIndex(InSubIndex)
	{}
};

struct HCurveEditorLabelBkgProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();
	HCurveEditorLabelBkgProxy(): HHitProxy(HPP_UI) {}
};

struct HCurveEditorHideSubCurveProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32 CurveIndex;
	int32 SubCurveIndex;

	HCurveEditorHideSubCurveProxy(int32 InCurveIndex, int32 InSubCurveIndex) :
	HHitProxy(HPP_UI),
		CurveIndex(InCurveIndex),
		SubCurveIndex(InSubCurveIndex)
	{}
};
