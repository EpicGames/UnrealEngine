// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorSharedData.h"

#include "Engine/InterpCurveEdSetup.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Preferences/CurveEdOptions.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

FCurveEditorSharedData::FCurveEditorSharedData(UInterpCurveEdSetup* InEdSetup)
	: LabelEntryHeight(36)
{
	EditorOptions = NewObject<UCurveEdOptions>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UCurveEdOptions::StaticClass(), FName(TEXT("EditorOptions"))));
	check(EditorOptions);
	MinViewRange = EditorOptions->MinViewRange;
	MaxViewRange = EditorOptions->MaxViewRange;

	EdSetup = InEdSetup;

	LabelContentBoxHeight = 0;

	RightClickCurveIndex = INDEX_NONE;
	RightClickCurveSubIndex = INDEX_NONE;

	EdMode = CEM_Pan;

	bShowPositionMarker = false;
	MarkerPosition = 0.f;
	MarkerColor = FColor::White;

	bShowEndMarker = false;
	EndMarkerPosition = 0.f;

	bShowRegionMarker = false;
	RegionStart = 0.f;
	RegionEnd = 0.f;
	RegionFillColor = FColor(255,255,255,16);

	bShowAllCurveTangents = false;

	NotifyObject = NULL;

	StartIn = EdSetup->Tabs[ EdSetup->ActiveTab ].ViewStartInput;
	EndIn = EdSetup->Tabs[ EdSetup->ActiveTab ].ViewEndInput;
	StartOut = EdSetup->Tabs[ EdSetup->ActiveTab ].ViewStartOutput;
	EndOut = EdSetup->Tabs[ EdSetup->ActiveTab ].ViewEndOutput;
}

FCurveEditorSharedData::~FCurveEditorSharedData()
{

}

void FCurveEditorSharedData::SetCurveView(float InStartIn, float InEndIn, float InStartOut, float InEndOut)
{
	// Ensure we are not zooming too big or too small...
	float InSize = InEndIn - InStartIn;
	float OutSize = InEndOut - InStartOut;
	if(InSize < MinViewRange  || InSize > MaxViewRange || OutSize < MinViewRange || OutSize > MaxViewRange)
	{
		return;
	}

	StartIn		= EdSetup->Tabs[ EdSetup->ActiveTab ].ViewStartInput	= InStartIn;
	EndIn		= EdSetup->Tabs[ EdSetup->ActiveTab ].ViewEndInput		= InEndIn;
	StartOut	= EdSetup->Tabs[ EdSetup->ActiveTab ].ViewStartOutput	= InStartOut;
	EndOut		= EdSetup->Tabs[ EdSetup->ActiveTab ].ViewEndOutput		= InEndOut;
}
