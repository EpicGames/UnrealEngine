// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Math/Color.h"

#define CURVEED_MAX_CURVES 6

#define CURVEEDENTRY_HIDECURVE_BIT					(0x00000001)		// Used by macros only
#define CURVEEDENTRY_HIDECURVE(x)					((x) & CURVEEDENTRY_HIDECURVE_BIT)
#define CURVEEDENTRY_TOGGLE_HIDECURVE(x)			((x) = ((x) ^ CURVEEDENTRY_HIDECURVE_BIT))
#define CURVEEDENTRY_SET_HIDECURVE(x, flg)			((x) = ((flg) ? ((x) | CURVEEDENTRY_HIDECURVE_BIT) : ((x) &~ CURVEEDENTRY_HIDECURVE_BIT)))

#define CURVEEDENTRY_HIDESUBCURVE_BIT(idx)			(1 << ((idx) + 1))	// Used by macros only
#define CURVEEDENTRY_HIDESUBCURVE(x, idx)			((x) & CURVEEDENTRY_HIDESUBCURVE_BIT(idx))
#define CURVEEDENTRY_TOGGLE_HIDESUBCURVE(x, idx)	((x) = ((x) ^ CURVEEDENTRY_HIDESUBCURVE_BIT(idx)))
#define CURVEEDENTRY_SET_HIDESUBCURVE(x, idx, flg)	((x) = ((flg) ? ((x) | CURVEEDENTRY_HIDESUBCURVE_BIT(idx)) : ((x) &~ CURVEEDENTRY_HIDESUBCURVE_BIT(idx))))

#define CURVEEDENTRY_SELECTED_BIT					(0x80000000)		// Used by macros only
#define CURVEEDENTRY_SELECTED(x)					((x) & CURVEEDENTRY_SELECTED_BIT)
#define CURVEEDENTRY_TOGGLE_SELECTED(x)				((x) = ((x) ^ CURVEEDENTRY_SELECTED_BIT))
#define CURVEEDENTRY_SET_SELECTED(x, flg)			((x) = ((flg) ? ((x) | CURVEEDENTRY_SELECTED_BIT) : ((x) &~ CURVEEDENTRY_SELECTED_BIT)))

class FCurveEdNotifyInterface;
class UCurveEdOptions;
class UInterpCurveEdSetup;

/*-----------------------------------------------------------------------------
   FCurveEditorModKey
-----------------------------------------------------------------------------*/

struct FCurveEditorModKey
{
	int32 CurveIndex;
	int32 KeyIndex;

	FCurveEditorModKey(int32 InCurveIndex, int32 InKeyIndex)
	{
		CurveIndex = InCurveIndex;
		KeyIndex = InKeyIndex;
	}

	bool operator==(const FCurveEditorModKey& Other) const
	{
		return((CurveIndex == Other.CurveIndex) && (KeyIndex == Other.KeyIndex));
	}
};


/*-----------------------------------------------------------------------------
   FCurveEditorSelectedKey
-----------------------------------------------------------------------------*/

struct FCurveEditorSelectedKey
{
	int32 CurveIndex;
	int32 SubIndex;
	int32 KeyIndex;
	float UnsnappedIn;
	float UnsnappedOut;

	FCurveEditorSelectedKey(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex)
	{
		CurveIndex = InCurveIndex;
		SubIndex = InSubIndex;
		KeyIndex = InKeyIndex;
		UnsnappedIn = 0.0f;
		UnsnappedOut = 0.0f;
	}

	bool operator==(const FCurveEditorSelectedKey& Other) const
	{
		if(CurveIndex == Other.CurveIndex &&
			SubIndex == Other.SubIndex &&
			KeyIndex == Other.KeyIndex)
		{
			return true;
		}
		else
		{
			return false;
		}
	}
};


/*-----------------------------------------------------------------------------
   FCurveEditorSharedData
-----------------------------------------------------------------------------*/

class FCurveEditorSharedData
{
public:
	/** Constructor/Destructor */
	FCurveEditorSharedData(UInterpCurveEdSetup* InEdSetup);
	virtual ~FCurveEditorSharedData();

	/** Sets up the viewing region */
	void SetCurveView(float StartIn, float EndIn, float StartOut, float EndOut);
	
	/** Mouse drag mode types */
	enum ECurveEdMode
	{
		CEM_Pan,
		CEM_Zoom
	};

public:
	/** Object for working with tabs and distribution data */
	UInterpCurveEdSetup* EdSetup;

	/** Object containing curve editor configuration info */
	UCurveEdOptions* EditorOptions;

	/** Object to be notified when changes are made to the curve editor */
	FCurveEdNotifyInterface* NotifyObject;

	/** Pan or zoom mode */
	ECurveEdMode EdMode;

	/** Currently selected keys */
	TArray<FCurveEditorSelectedKey> SelectedKeys;

	/** Index of the curve that was right-clicked */
	int32 RightClickCurveIndex;
	int32 RightClickCurveSubIndex;

	/** Individual and total draw heights for labels */
	const int32	LabelEntryHeight;
	int32 LabelContentBoxHeight;

	/** Draw info */
	float StartIn;
	float EndIn;
	float StartOut;
	float EndOut;
	float MaxViewRange;
	float MinViewRange;
	bool bShowPositionMarker;
	float MarkerPosition;
	FColor MarkerColor;
	bool bShowEndMarker;
	float EndMarkerPosition;
	bool bShowRegionMarker;
	float RegionStart;
	float RegionEnd;
	FColor RegionFillColor;
	bool bShowAllCurveTangents;
};
