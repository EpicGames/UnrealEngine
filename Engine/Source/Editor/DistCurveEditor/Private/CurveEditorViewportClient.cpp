// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorViewportClient.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CurveEditorHitProxies.h"
#include "CurveEditorSharedData.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Engine.h"
#include "Engine/InterpCurveEdSetup.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericWindow.h"
#include "HAL/PlatformCrt.h"
#include "HitProxies.h"
#include "IDistCurveEditor.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/CurveEdInterface.h"
#include "Math/IntRect.h"
#include "Math/InterpCurvePoint.h"
#include "Math/TranslationMatrix.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Parse.h"
#include "Preferences/CurveEdOptions.h"
#include "Preferences/UnrealEdOptions.h"
#include "SCurveEditorViewport.h"
#include "SDistributionCurveEditor.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Slate/SceneViewport.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/SWindow.h"

class UObject;

FCurveEditorViewportClient::FCurveEditorViewportClient(TWeakPtr<SDistributionCurveEditor> InCurveEditor, TWeakPtr<SCurveEditorViewport> InCurveEditorViewport)
	: CurveEditorPtr(InCurveEditor)
	, CurveEditorViewportPtr(InCurveEditorViewport)
	, LabelWidth(200)
	, ColorKeyWidth(6)
	, ZoomSpeed(0.1f)
	, MouseZoomSpeed(0.015f)
	, HandleLength(30.f)
	, FitMargin(0.1f)
	, CurveDrawRes(5)
{
	check(CurveEditorPtr.IsValid() && CurveEditorViewportPtr.IsValid());

	SharedData = CurveEditorPtr.Pin()->GetSharedData();
	
	LabelOrigin2D.X	= 0;
	LabelOrigin2D.Y	= 0;

	DragStartMouseX = 0;
	DragStartMouseY = 0;
	OldMouseX = 0;
	OldMouseY = 0;

	bPanning = false;
	bDraggingHandle = false;
	bMouseDown = false;
	bBegunMoving = false;
	MovementAxisLock = AxisLock_None;
	bBoxSelecting = false;
	bKeyAdded = false;
	BoxStartX = 0;
	BoxStartY = 0;
	BoxEndX = 0;
	BoxEndY = 0;
	DistanceDragged = 0;

	BackgroundColor = SharedData->EditorOptions->BackgroundColor;
	LabelColor = SharedData->EditorOptions->LabelColor;
	SelectedLabelColor = SharedData->EditorOptions->SelectedLabelColor;
	GridColor = SharedData->EditorOptions->GridColor;
	GridTextColor = SharedData->EditorOptions->GridTextColor;
	LabelBlockBkgColor = SharedData->EditorOptions->LabelBlockBkgColor;
	SelectedKeyColor = SharedData->EditorOptions->SelectedKeyColor;

	MouseOverCurveIndex = INDEX_NONE;
	MouseOverSubIndex = INDEX_NONE;
	MouseOverKeyIndex = INDEX_NONE;

	HandleCurveIndex = INDEX_NONE;
	HandleSubIndex = INDEX_NONE;
	HandleKeyIndex = INDEX_NONE;
	bHandleArriving = false;

	bSnapToFrames = false;
	
	bSnapEnabled = false;
	InSnapAmount = 1.f;

	FSlateApplication::Get().OnWindowDPIScaleChanged().AddRaw(this, &FCurveEditorViewportClient::HandleWindowDPIScaleChanged);
}

FCurveEditorViewportClient::~FCurveEditorViewportClient()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnWindowDPIScaleChanged().RemoveAll(this);
	}
}

void FCurveEditorViewportClient::Draw(FViewport* Viewport, FCanvas* Canvas)
{
	if (!CurveEditorPtr.IsValid())
	{
		return;
	}
	
	FVector2D ScrollBarPos = GetViewportScrollBarPositions();
	
	UpdateScrollBars();

	LabelOrigin2D.Y = -ScrollBarPos.Y;

	if(Viewport->GetSizeXY().X <= LabelWidth || Viewport->GetSizeXY().Y <= 1)
	{
		return;
	}

	Canvas->Clear(BackgroundColor);

	CurveViewX = Viewport->GetSizeXY().X - LabelWidth;
	CurveViewY = Viewport->GetSizeXY().Y;

	PixelsPerIn = CurveViewX/(SharedData->EndIn - SharedData->StartIn);
	PixelsPerOut = CurveViewY/(SharedData->EndOut - SharedData->StartOut);

	// Draw background grid.
	DrawGrid(Viewport, Canvas);

	// Draw selected-region if desired.
	if(SharedData->bShowRegionMarker)
	{
		FIntPoint RegionStartPos = CalcScreenPos(FVector2D(SharedData->RegionStart, 0));
		FIntPoint RegionEndPos = CalcScreenPos(FVector2D(SharedData->RegionEnd, 0));

		Canvas->DrawTile(RegionStartPos.X, 0, RegionEndPos.X - RegionStartPos.X, CurveViewY, 0.f, 0.f, 1.f, 1.f, SharedData->RegionFillColor);
	}

	// Draw each curve
	for(int32 i=0; i<SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num(); i++)
	{
		// Draw curve itself.
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[i];
		if(!CURVEEDENTRY_HIDECURVE(Entry.bHideCurve))
		{
			DrawEntry(Viewport, Canvas, Entry, i);
		}
	}

	// Draw key background block down left hand side.
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(new HCurveEditorLabelBkgProxy());
	}
	Canvas->DrawTile( 0, 0, LabelWidth, CurveViewY, 0.f, 0.f, 1.f, 1.f, LabelBlockBkgColor,NULL,false);
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(NULL);
	}

	// Draw key entry for each curve
	Canvas->PushRelativeTransform(FTranslationMatrix(FVector(LabelOrigin2D.X,LabelOrigin2D.Y,0)));
	int32 CurrentKeyY = 0;
	for(int32 i=0; i<SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num(); i++)
	{
		// Draw key entry
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[i];

		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
		if (EdInterface)
		{
			// Draw background, color-square and text
			if (Canvas->IsHitTesting())
			{
				Canvas->SetHitProxy(new HCurveEditorLabelProxy(i));
			}
			if (CURVEEDENTRY_SELECTED(Entry.bHideCurve))
			{
				Canvas->DrawTile(0, CurrentKeyY, LabelWidth, SharedData->LabelEntryHeight, 0.f, 0.f, 1.f, 1.f, SelectedLabelColor);
			}
			else
			{
				Canvas->DrawTile( 0, CurrentKeyY, LabelWidth, SharedData->LabelEntryHeight, 0.f, 0.f, 1.f, 1.f, LabelColor);
			}
			Canvas->DrawTile( 0, CurrentKeyY, ColorKeyWidth, SharedData->LabelEntryHeight, 0.f, 0.f, 1.f, 1.f, Entry.CurveColor);
			Canvas->DrawShadowedString( ColorKeyWidth+3, CurrentKeyY+4, *(Entry.CurveName), GEngine->GetSmallFont(), FLinearColor::White);

			if (Canvas->IsHitTesting())
			{
				Canvas->SetHitProxy(NULL);
			}

			// Draw hide/unhide button
			FColor ButtonColor = CURVEEDENTRY_HIDECURVE(Entry.bHideCurve) ? FColor(112,112,112) : FColor(255,200,0);
			if (Canvas->IsHitTesting())
			{
				Canvas->SetHitProxy(new HCurveEditorHideCurveProxy(i));
			}
			Canvas->DrawTile( LabelWidth - 12, CurrentKeyY + SharedData->LabelEntryHeight - 12, 8, 8, 0.f, 0.f, 1.f, 1.f, FLinearColor::Black);
			Canvas->DrawTile( LabelWidth - 11, CurrentKeyY + SharedData->LabelEntryHeight - 11, 6, 6, 0.f, 0.f, 1.f, 1.f, ButtonColor);
			if (Canvas->IsHitTesting())
			{
				Canvas->SetHitProxy(NULL);
			}

			// Draw the sub-curve hide/unhide buttons
			int32		SubCurveButtonOffset	= 8;

			int32 NumSubs = EdInterface->GetNumSubCurves();
			for (int32 ii = 0; ii < NumSubs; ii++)
			{
				ButtonColor = EdInterface->GetSubCurveButtonColor(ii, !!CURVEEDENTRY_HIDESUBCURVE(Entry.bHideCurve, ii));

				if (Canvas->IsHitTesting())
				{
					Canvas->SetHitProxy(new HCurveEditorHideSubCurveProxy(i, ii));
				}
				Canvas->DrawTile(SubCurveButtonOffset + 0, CurrentKeyY + SharedData->LabelEntryHeight - 12, 8, 8, 0.f, 0.f, 1.f, 1.f, FLinearColor::Black);
				Canvas->DrawTile(SubCurveButtonOffset + 1, CurrentKeyY + SharedData->LabelEntryHeight - 11, 6, 6, 0.f, 0.f, 1.f, 1.f, ButtonColor);

				SubCurveButtonOffset +=12;

				if (Canvas->IsHitTesting())
				{
					Canvas->SetHitProxy(NULL);
				}
			}
		}

		CurrentKeyY += 	SharedData->LabelEntryHeight;

		// Draw line under each key entry
		Canvas->DrawTile( 0, CurrentKeyY-1, LabelWidth, 1, 0.f, 0.f, 0.f, 0.f, FLinearColor::Black);
	}

	Canvas->PopTransform();

	// Draw line above top-most key entry.
	Canvas->DrawTile( 0, 0, LabelWidth, 1, 0.f, 0.f, 0.f, 0.f, FLinearColor::Black);

	// Line down right of key.
	Canvas->DrawTile(LabelWidth, 0, 1, CurveViewY, 0.f, 0.f, 0.f, 0.f, FLinearColor::Black);

	// Draw box-selection region
	if(bBoxSelecting)
	{
		int32 MinX = FMath::Min(BoxStartX, BoxEndX);
		int32 MinY = FMath::Min(BoxStartY, BoxEndY);
		int32 MaxX = FMath::Max(BoxStartX, BoxEndX);
		int32 MaxY = FMath::Max(BoxStartY, BoxEndY);
		FCanvasBoxItem BoxItem( FVector2D(MinX, MinY), FVector2D(MaxX-MinX, MaxY-MinY) );
		BoxItem.SetColor( FLinearColor::Red );
		Canvas->DrawItem( BoxItem );
	}

	FCanvasLineItem LineItem;
	if(SharedData->bShowPositionMarker)
	{
		FIntPoint MarkerScreenPos = CalcScreenPos(FVector2D(SharedData->MarkerPosition, 0));
		if(MarkerScreenPos.X >= LabelWidth)
		{
			LineItem.SetColor( SharedData->MarkerColor );
			LineItem.Draw( Canvas, FVector2D(MarkerScreenPos.X, 0), FVector2D(MarkerScreenPos.X, CurveViewY) );
		}
	}

	if(SharedData->bShowEndMarker)
	{
		FIntPoint EndScreenPos = CalcScreenPos(FVector2D(SharedData->EndMarkerPosition, 0));
		if(EndScreenPos.X >= LabelWidth)
		{
			LineItem.SetColor( FLinearColor::White );
			LineItem.Draw( Canvas, FVector2D(EndScreenPos.X, 0), FVector2D(EndScreenPos.X, CurveViewY) );
		}
	}
}

bool FCurveEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{	
	bool bHandled = false;

	FViewport* Viewport = EventArgs.Viewport;
	const FKey& Key = EventArgs.Key;
	const EInputEvent Event = EventArgs.Event;

	bool bCtrlDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	bool bShiftDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
	bool bAltDown = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);

	int32 HitX = Viewport->GetMouseX();
	int32 HitY = Viewport->GetMouseY();
	FIntPoint MousePos = FIntPoint(HitX, HitY);

	if(Key == EKeys::LeftMouseButton)
	{
		bHandled = true;

		if(SharedData->EdMode == FCurveEditorSharedData::CEM_Pan)
		{
			if(Event == IE_Pressed)
			{
				HHitProxy*	HitResult = Viewport->GetHitProxy(HitX,HitY);
				if(HitResult && !ProcessNonGraphHit( HitResult ))
				{
					if(HitResult->IsA(HCurveEditorKeyProxy::StaticGetType()))
					{
						int32 CurveIndex = ((HCurveEditorKeyProxy*)HitResult)->CurveIndex;
						int32 SubIndex = ((HCurveEditorKeyProxy*)HitResult)->SubIndex;
						int32 KeyIndex = ((HCurveEditorKeyProxy*)HitResult)->KeyIndex;

						if (!bCtrlDown && !bShiftDown)
						{
							SharedData->SelectedKeys.Empty();
						}
						
						if (!bShiftDown)
						{
							if (KeyIsInSelection(CurveIndex, SubIndex, KeyIndex))
							{
								RemoveKeyFromSelection(CurveIndex, SubIndex, KeyIndex);
							}
							else
							{
								AddKeyToSelection(CurveIndex, SubIndex, KeyIndex);
							}
						}
						else
						{
							const bool bSelectKeys = !KeyIsInSelection(CurveIndex, SubIndex, KeyIndex);

							FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[CurveIndex];
							if(!CURVEEDENTRY_HIDECURVE(Entry.bHideCurve))
							{
								FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
								if(EdInterface)
								{
									for (int32 SubIdx = 0; SubIdx < EdInterface->GetNumSubCurves(); SubIdx++)
									{
										// Holding SHIFT while clicking will toggle all keys at that point.
										if (bSelectKeys)
										{
											AddKeyToSelection(CurveIndex, SubIdx, KeyIndex);
										}
										else
										{
											RemoveKeyFromSelection(CurveIndex, SubIdx, KeyIndex);
										}
									}
								}
							}
						}

						BeginMoveSelectedKeys();
						bBegunMoving = true;
						MovementAxisLock = AxisLock_None;
					}
					else if(HitResult->IsA(HCurveEditorKeyHandleProxy::StaticGetType()))
					{
						HCurveEditorKeyHandleProxy* HandleProxy = (HCurveEditorKeyHandleProxy*)HitResult;

						HandleCurveIndex = ((HCurveEditorKeyHandleProxy*)HitResult)->CurveIndex;
						HandleSubIndex = ((HCurveEditorKeyHandleProxy*)HitResult)->SubIndex;
						HandleKeyIndex = ((HCurveEditorKeyHandleProxy*)HitResult)->KeyIndex;
						bHandleArriving = ((HCurveEditorKeyHandleProxy*)HitResult)->bArriving;

						// Notify a containing tool we are about to move a handle
						if (SharedData->NotifyObject)
						{
							FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[HandleProxy->CurveIndex];
							TArray<UObject*> CurvesAboutToChange;
							CurvesAboutToChange.Add(Entry.CurveObject);
							SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);
						}

						bDraggingHandle = true;
					}
					else if(HitResult->IsA(HCurveEditorLineProxy::StaticGetType()))
					{
						if(bCtrlDown)
						{
							// Clicking on the line creates a new key.
							int32 CurveIndex = ((HCurveEditorLineProxy*)HitResult)->CurveIndex;
							int32 SubIndex = ((HCurveEditorLineProxy*)HitResult)->SubIndex;

							int32 NewKeyIndex = AddNewKeypoint(CurveIndex, SubIndex, FIntPoint(HitX, HitY));

							CurveEditorViewportPtr.Pin()->RefreshViewport();

							// Select just this new key straight away so we can drag stuff around.
							if(NewKeyIndex != INDEX_NONE)
							{
								SharedData->SelectedKeys.Empty();
								AddKeyToSelection(CurveIndex, SubIndex, NewKeyIndex);
								bKeyAdded = true;
							}
						}
						else
						{
							bPanning = true;
						}
					}
				}
				else if(bCtrlDown && bAltDown)
				{
					BoxStartX = BoxEndX = HitX;
					BoxStartY = BoxEndY = HitY;

					bBoxSelecting = true;
				}
				else if (bCtrlDown)
				{
					BeginMoveSelectedKeys();
					bBegunMoving = true;
					MovementAxisLock = AxisLock_None;
				}
				else
				{
					bPanning = true;
				}

				DragStartMouseX = OldMouseX = HitX;
				DragStartMouseY = OldMouseY = HitY;
				bMouseDown = true;
				DistanceDragged = 0;
				Viewport->LockMouseToViewport(true);
				Viewport->InvalidateHitProxy();
			}
			else if(Event == IE_Released)
			{
				if(!bKeyAdded)
				{
					if(bBoxSelecting)
					{
						int32 MinX = FMath::Max(0, FMath::Min(BoxStartX, BoxEndX));
						int32 MinY = FMath::Max(0, FMath::Min(BoxStartY, BoxEndY));
						int32 MaxX = FMath::Min(Viewport->GetSizeXY().X - 1, FMath::Max(BoxStartX, BoxEndX));
						int32 MaxY = FMath::Min(Viewport->GetSizeXY().Y - 1, FMath::Max(BoxStartY, BoxEndY));
						int32 TestSizeX = MaxX - MinX + 1;
						int32 TestSizeY = MaxY - MinY + 1;

						// We read back the hit proxy map for the required region.
						TArray<HHitProxy*> ProxyMap;
						Viewport->GetHitProxyMap(FIntRect(MinX, MinY, MaxX + 1, MaxY + 1), ProxyMap);

						TArray<FCurveEditorSelectedKey> NewSelection;

						// Find any keypoint hit proxies in the region - add the keypoint to selection.
						for(int32 Y=0; Y < TestSizeY; Y++)
						{
							for(int32 X=0; X < TestSizeX; X++)
							{
								HHitProxy* HitProxy = ProxyMap[Y * TestSizeX + X];

								if(HitProxy && HitProxy->IsA(HCurveEditorKeyProxy::StaticGetType()))
								{
									int32 CurveIndex = ((HCurveEditorKeyProxy*)HitProxy)->CurveIndex;
									int32 SubIndex = ((HCurveEditorKeyProxy*)HitProxy)->SubIndex;
									int32 KeyIndex = ((HCurveEditorKeyProxy*)HitProxy)->KeyIndex;								

									NewSelection.Add(FCurveEditorSelectedKey(CurveIndex, SubIndex, KeyIndex));
								}
							}
						}

						// If shift is down, don't empty, just add to selection.
						if(!bShiftDown)
						{
							SharedData->SelectedKeys.Empty();
						}

						// Iterate over array adding each to selection.
						for(int32 i=0; i<NewSelection.Num(); i++)
						{
							AddKeyToSelection(NewSelection[i].CurveIndex, NewSelection[i].SubIndex, NewSelection[i].KeyIndex);
						}
					}
					else
					{
						HHitProxy*	HitResult = Viewport->GetHitProxy(HitX,HitY);

						if(!HitResult && DistanceDragged < 4)
						{
							SharedData->SelectedKeys.Empty();
						}
					}
				}

				if(bBegunMoving)
				{
					EndMoveSelectedKeys();
					bBegunMoving = false;

					// Make sure that movement axis lock is no longer enabled
					MovementAxisLock = AxisLock_None;
				}
			}
		}
		else if(SharedData->EdMode == FCurveEditorSharedData::CEM_Zoom)
		{
			if(Event == IE_Pressed)
			{
				HHitProxy*	HitResult = Viewport->GetHitProxy(HitX,HitY);
				if(HitResult)
				{
					ProcessNonGraphHit( HitResult );
				}
			}
		}

		if(Event == IE_Released)
		{
			bMouseDown = false;
			DistanceDragged = 0;
			bPanning = false;
			// Notify a containing tool we are about to move a handle
			if (bDraggingHandle && SharedData->NotifyObject)
			{
				SharedData->NotifyObject->PostEditCurve();
			}
			bDraggingHandle = false;
			bBoxSelecting = false;
			bKeyAdded = false;

			Viewport->LockMouseToViewport(false);
			Viewport->InvalidateHitProxy();
		}
	}
	else if(Key == EKeys::RightMouseButton)
	{
		bHandled = true;

		if(Event == IE_Released)
		{
			HHitProxy*	HitResult = Viewport->GetHitProxy(HitX,HitY);
			if(HitResult)
			{
				if(HitResult->IsA(HCurveEditorLabelProxy::StaticGetType()))
				{
					SharedData->RightClickCurveIndex = ((HCurveEditorLabelProxy*)HitResult)->CurveIndex;

					CurveEditorPtr.Pin()->OpenLabelMenu();
				}
				else if(HitResult->IsA(HCurveEditorKeyProxy::StaticGetType()))
				{
					if(SharedData->EdMode == FCurveEditorSharedData::CEM_Pan)
					{
						int32 CurveIndex = ((HCurveEditorKeyProxy*)HitResult)->CurveIndex;
						int32 SubIndex = ((HCurveEditorKeyProxy*)HitResult)->SubIndex;
						int32 KeyIndex = ((HCurveEditorKeyProxy*)HitResult)->KeyIndex;

						if(!KeyIsInSelection(CurveIndex, SubIndex, KeyIndex))
						{
							SharedData->SelectedKeys.Empty();
							AddKeyToSelection(CurveIndex, SubIndex, KeyIndex);
						}

						CurveEditorPtr.Pin()->OpenKeyMenu();
					}
				}
				else if(HitResult->IsA(HCurveEditorLineProxy::StaticGetType()))
				{
					SharedData->RightClickCurveIndex = ((HCurveEditorLineProxy*)HitResult)->CurveIndex;
					SharedData->RightClickCurveSubIndex = ((HCurveEditorLineProxy*)HitResult)->SubIndex;

					CurveEditorPtr.Pin()->OpenCurveMenu();
				}
			}
			else
			{
				if (SharedData->EdMode != FCurveEditorSharedData::CEM_Zoom)
				{
					// Show the general context menu
					CurveEditorPtr.Pin()->OpenGeneralMenu();
				}
			}
		}
	}
	else if (((Key == EKeys::MouseScrollDown) || (Key == EKeys::MouseScrollUp && Event == IE_Pressed)) && (Event == IE_Pressed))
	{
		bHandled = true;

		//down = zoom out.
		float Direction = (Key == EKeys::MouseScrollDown) ? -1.0f : 1.0f;
		float SizeIn = SharedData->EndIn - SharedData->StartIn;
		float DeltaIn = ZoomSpeed * SizeIn * Direction;

		float SizeOut = SharedData->EndOut - SharedData->StartOut;
		float DeltaOut = ZoomSpeed * SizeOut * Direction;

		float NewStartIn = SharedData->StartIn + DeltaIn;
		float NewEndIn = SharedData->EndIn - DeltaIn;
		float NewStartOut = SharedData->StartOut + DeltaOut;
		float NewEndOut = SharedData->EndOut - DeltaOut;

		if (GetDefault<ULevelEditorViewportSettings>()->bCenterZoomAroundCursor)
		{
			float MouseX = Viewport->GetMouseX()-LabelWidth;
			float MouseY = Viewport->GetMouseY();
			float ViewportWidth = Viewport->GetSizeXY().X-LabelWidth;
			float ViewportHeight = Viewport->GetSizeXY().Y;			
			check(ViewportWidth > 0);
			check(ViewportHeight > 0);

			//(Keep left side the same) - at viewport x = 0, offset is -DeltaIn
			//(Stay Centered) - at viewport x = viewport->Getwidth/2, offset is 0
			//(Keep right side the same) - at viewport x = viewport->Getwidth, offset is DeltaIn
			float OffsetX = ((MouseX / ViewportWidth)-.5f)*2.0f*DeltaIn;
			//negate why to account for y being inverted
			float OffsetY = -((MouseY / ViewportHeight)-.5f)*2.0f*DeltaOut;

			NewStartIn += OffsetX;
			NewEndIn += OffsetX;
			NewStartOut += OffsetY;
			NewEndOut += OffsetY;
		}

		SharedData->SetCurveView(NewStartIn, NewEndIn, NewStartOut, NewEndOut);
		Viewport->Invalidate();
	}
	else if(Event == IE_Pressed)
	{
		if ( Key == EKeys::Platform_Delete )
		{
			CurveEditorPtr.Pin()->OnDeleteKeys();
			bHandled = true;
		}
		else if(Key == EKeys::Z && bCtrlDown)
		{
			if(SharedData->NotifyObject)
			{
				SharedData->NotifyObject->DesireUndo();
			}
			bHandled = true;
		}
		else if(Key == EKeys::Y && bCtrlDown)
		{
			if(SharedData->NotifyObject)
			{
				SharedData->NotifyObject->DesireRedo();
			}
			bHandled = true;
		}
		else if(Key == EKeys::Z)
		{
			if(!bBoxSelecting && !bBegunMoving && !bDraggingHandle)
			{
				SharedData->EdMode = FCurveEditorSharedData::CEM_Zoom;
			}
			bHandled = true;
		}
		else if ((Key == EKeys::F) && bCtrlDown)
		{
			if(!bBoxSelecting && !bBegunMoving && !bDraggingHandle)
			{
				CurveEditorPtr.Pin()->OnFit();
			}
			bHandled = true;
		}
		else
		{
			// Handle hotkey bindings.
			UUnrealEdOptions* UnrealEdOptions = GUnrealEd->GetUnrealEdOptions();

			if(UnrealEdOptions)
			{
				FString Cmd = UnrealEdOptions->GetExecCommand(Key, bAltDown, bCtrlDown, bShiftDown, TEXT("CurveEditor"));

				if(Cmd.Len())
				{
					Exec(*Cmd);
					bHandled = true;
				}
			}
		}
	}
	else if(Event == IE_Released)
	{
		if(Key == EKeys::Z)
		{
			SharedData->EdMode = FCurveEditorSharedData::CEM_Pan;
			bHandled = true;
		}
	}

	return bHandled;
}

void FCurveEditorViewportClient::MouseMove(FViewport* Viewport, int32 X, int32 Y)
{
	bool bCtrlDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	bool bShiftDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);

	int32 DeltaX = OldMouseX - X;
	OldMouseX = X;

	int32 DeltaY = OldMouseY - Y;
	OldMouseY = Y;

	// Update mouse-over keypoint.
	HHitProxy* HitResult = Viewport->GetHitProxy(X,Y);
	if(HitResult && HitResult->IsA(HCurveEditorKeyProxy::StaticGetType()))
	{
		MouseOverCurveIndex = ((HCurveEditorKeyProxy*)HitResult)->CurveIndex;
		MouseOverSubIndex = ((HCurveEditorKeyProxy*)HitResult)->SubIndex;
		MouseOverKeyIndex = ((HCurveEditorKeyProxy*)HitResult)->KeyIndex;
	}
	else
	{
		MouseOverCurveIndex = INDEX_NONE;
		MouseOverSubIndex = INDEX_NONE;
		MouseOverKeyIndex = INDEX_NONE;
	}

	// If in panning mode, do moving/panning stuff.
	if(SharedData->EdMode == FCurveEditorSharedData::CEM_Pan)
	{
		if(bMouseDown)
		{
			// Update total milage of mouse cursor while button is pressed.
			DistanceDragged += (FMath::Abs<int32>(DeltaX) + FMath::Abs<int32>(DeltaY));

			// Distance mouse just moved in 'curve' units.
			float DeltaIn = (DeltaX / PixelsPerIn)/GetDPIScale();
			float DeltaOut = (-DeltaY / PixelsPerOut)/GetDPIScale();

			// If we are panning around, update the Start/End In/Out values for this view.
			if(bDraggingHandle)
			{
				FVector2D HandleVal = CalcValuePoint(FIntPoint(X,Y));
				MoveCurveHandle(HandleVal);
			}
			else if(bBoxSelecting)
			{
				BoxEndX = X;
				BoxEndY = Y;
			}
			else if(bPanning)
			{
				SharedData->SetCurveView(SharedData->StartIn + DeltaIn, SharedData->EndIn + DeltaIn, SharedData->StartOut + DeltaOut, SharedData->EndOut + DeltaOut);
			}
			else if (bBegunMoving && DistanceDragged > 4)
			{
				// If the Shift key is held down, then we'll
				// lock key movement to the specified axis
				if(bShiftDown && MovementAxisLock == AxisLock_None)
				{
					// Set movement axis lock based on the user's mouse position
					if(FMath::Abs(X - DragStartMouseX) > FMath::Abs(Y - DragStartMouseY))
					{
						MovementAxisLock = AxisLock_Horizontal;
					}
					else
					{
						MovementAxisLock = AxisLock_Vertical;
					}
				}

				MoveSelectedKeys(
					MovementAxisLock == AxisLock_Vertical ? 0 : -DeltaIn,
					MovementAxisLock == AxisLock_Horizontal ? 0 : -DeltaOut);
			}
		}
	}
	// Otherwise we are in zooming mode, so look at mouse buttons and update viewport size.
	else if(SharedData->EdMode == FCurveEditorSharedData::CEM_Zoom)
	{
		bool bLeftMouseDown = Viewport->KeyState(EKeys::LeftMouseButton);
		bool bRightMouseDown = Viewport->KeyState(EKeys::RightMouseButton);

		float ZoomDeltaIn = 0.f;
		if(bRightMouseDown)
		{
			float SizeIn = SharedData->EndIn - SharedData->StartIn;
			ZoomDeltaIn = MouseZoomSpeed * SizeIn * FMath::Clamp<int32>(DeltaX - DeltaY, -5, 5);		
		}

		float ZoomDeltaOut = 0.f;
		if(bLeftMouseDown)
		{
			float SizeOut = SharedData->EndOut - SharedData->StartOut;
			ZoomDeltaOut = MouseZoomSpeed * SizeOut * FMath::Clamp<int32>(-DeltaY + DeltaX, -5, 5);
		}

		SharedData->SetCurveView(SharedData->StartIn - ZoomDeltaIn, SharedData->EndIn + ZoomDeltaIn, SharedData->StartOut - ZoomDeltaOut, SharedData->EndOut + ZoomDeltaOut);
	}

	Viewport->InvalidateDisplay();
}

void FCurveEditorViewportClient::CapturedMouseMove(FViewport* Viewport, int32 X, int32 Y)
{
	MouseMove(Viewport, X, Y);
}

bool FCurveEditorViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	if (Args.Key == EKeys::MouseX || Args.Key == EKeys::MouseY)
	{
		int32 X = Args.Viewport->GetMouseX();
		int32 Y = Args.Viewport->GetMouseY();
		MouseMove(Args.Viewport, X, Y);
		return true;
	}
	return false;
}

void FCurveEditorViewportClient::Exec(const TCHAR* Cmd)
{
	const TCHAR* Str = Cmd;

	if(FParse::Command(&Str, TEXT("CURVEEDITOR")))
	{
		if(FParse::Command(&Str, TEXT("ChangeInterpModeAUTO")))
		{
			CurveEditorPtr.Pin()->OnSetTangentType(CIM_CurveAuto);
		}
		if(FParse::Command(&Str, TEXT("ChangeInterpModeAUTOCLAMPED")))
		{
			CurveEditorPtr.Pin()->OnSetTangentType(CIM_CurveAutoClamped);
		}
		else if(FParse::Command(&Str, TEXT("ChangeInterpModeUSER")))
		{
			CurveEditorPtr.Pin()->OnSetTangentType(CIM_CurveUser);
		}
		else if(FParse::Command(&Str, TEXT("ChangeInterpModeBREAK")))
		{
			CurveEditorPtr.Pin()->OnSetTangentType(CIM_CurveBreak);
		}
		else if(FParse::Command(&Str, TEXT("ChangeInterpModeLINEAR")))
		{
			CurveEditorPtr.Pin()->OnSetTangentType(CIM_Linear);
		}
		else if(FParse::Command(&Str, TEXT("ChangeInterpModeCONSTANT")))
		{
			CurveEditorPtr.Pin()->OnSetTangentType(CIM_Constant);
		}
		else if(FParse::Command(&Str, TEXT("FitViewHorizontally")))
		{
			CurveEditorPtr.Pin()->OnFitHorizontally();
		}
		else if(FParse::Command(&Str, TEXT("FitViewVertically")))
		{
			CurveEditorPtr.Pin()->OnFitVertically();
		}
		// Multiple commands to support backwards compat. with old selected/all code paths
		else if(FParse::Command(&Str, TEXT("FitViewToAll")) ||
				FParse::Command(&Str, TEXT("FitViewToSelected")) ||
				FParse::Command(&Str, TEXT("FitView")))
		{
			CurveEditorPtr.Pin()->OnFit();
		}
	}
}

float FCurveEditorViewportClient::GetViewportVerticalScrollBarRatio() const
{
	if (SharedData->LabelContentBoxHeight == 0.0f)
	{
		return 1.0f;
	}

	float WidgetHeight = 1.0f;
	if (CurveEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		WidgetHeight = CurveEditorViewportPtr.Pin()->GetViewport()->GetSizeXY().Y;
	}
	
	return WidgetHeight / SharedData->LabelContentBoxHeight;
}

bool FCurveEditorViewportClient::ProcessNonGraphHit(HHitProxy* HitResult)
{
	check(HitResult);
	if(HitResult->IsA(HCurveEditorLabelProxy::StaticGetType()))
	{
		// Notify containing tool that a curve label was clicked on
		if(SharedData->NotifyObject != NULL)
		{
			const int32 CurveIndex = ((HCurveEditorLabelProxy*)HitResult)->CurveIndex;

			FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[ SharedData->EdSetup->ActiveTab ].Curves[ CurveIndex  ];

			SharedData->NotifyObject->OnCurveLabelClicked(Entry.CurveObject);
		}
		return true;
	}
	else if(HitResult->IsA(HCurveEditorHideCurveProxy::StaticGetType()))
	{
		int32 CurveIndex = ((HCurveEditorHideCurveProxy*)HitResult)->CurveIndex;

		ToggleCurveHidden(CurveIndex);
		return true;
	}
	else if (HitResult->IsA(HCurveEditorHideSubCurveProxy::StaticGetType()))
	{
		HCurveEditorHideSubCurveProxy* SubCurveProxy = (HCurveEditorHideSubCurveProxy*)HitResult;

		int32	CurveIndex		= SubCurveProxy->CurveIndex;
		int32	SubCurveIndex	= SubCurveProxy->SubCurveIndex;

		ToggleSubCurveHidden(CurveIndex, SubCurveIndex);
		return true;
	}
	return false;
}

void FCurveEditorViewportClient::UpdateScrollBars()
{
	SharedData->LabelContentBoxHeight = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num() * SharedData->LabelEntryHeight;
	
	if (CurveEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		float VRatio = GetViewportVerticalScrollBarRatio();
		float VDistFromBottom = CurveEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		
		if (VRatio < 1.0f)
		{
			if (VDistFromBottom < 1.0f)
			{
				CurveEditorViewportPtr.Pin()->GetVerticalScrollBar()->SetState(FMath::Clamp(1.0f - VRatio - VDistFromBottom, 0.0f, 1.0f), VRatio);
			}
			else
			{
				CurveEditorViewportPtr.Pin()->GetVerticalScrollBar()->SetState(0.0f, VRatio);
			}
		}
	}
}

void FCurveEditorViewportClient::ChangeViewportScrollBarPosition(EScrollDirection Direction)
{
	if (CurveEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		float Ratio = GetViewportVerticalScrollBarRatio();
		float DistFromBottom = CurveEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float OneMinusRatio = 1.0f - Ratio;
		float Diff = 0.1f * OneMinusRatio;

		if (Direction == Scroll_Down)
		{
			Diff *= -1.0f;
		}

		CurveEditorViewportPtr.Pin()->GetVerticalScrollBar()->SetState(FMath::Clamp(OneMinusRatio - DistFromBottom + Diff, 0.0f, OneMinusRatio), Ratio);

		CurveEditorViewportPtr.Pin()->RefreshViewport();
	}
}

FVector2D FCurveEditorViewportClient::GetViewportScrollBarPositions() const
{
	FVector2D Positions = FVector2D::ZeroVector;
	if (CurveEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		float VRatio = GetViewportVerticalScrollBarRatio();
		float VDistFromBottom = CurveEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();

		if ((CurveEditorViewportPtr.Pin()->GetVerticalScrollBar()->GetVisibility() == EVisibility::Visible) && VDistFromBottom < 1.0f)
		{
			Positions.Y = FMath::Clamp(1.0f - VRatio - VDistFromBottom, 0.0f, 1.0f) * SharedData->LabelContentBoxHeight;
		}
		else
		{
			Positions.Y = 0.0f;
		}
	}

	return Positions;
}

void FCurveEditorViewportClient::DrawEntry(FViewport* Viewport, FCanvas* Canvas, const FCurveEdEntry& Entry, int32 CurveIndex)
{
	FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
	if(!EdInterface)
	{
		return;
	}

	int32 NumSubs = EdInterface->GetNumSubCurves();
	int32 NumKeys = EdInterface->GetNumKeys();

	FCanvasLineItem LineItem;
	for(int32 SubIdx=0; SubIdx<NumSubs; SubIdx++)
	{
		if (CURVEEDENTRY_HIDESUBCURVE(Entry.bHideCurve, SubIdx))
		{
			continue;
		}

		FVector2D OldKey(0.f, 0.f);
		FIntPoint OldKeyPos(0, 0);

		// Draw curve
		for(int32 KeyIdx=0; KeyIdx<NumKeys; KeyIdx++)
		{
			FVector2D NewKey;
			NewKey.X = EdInterface->GetKeyIn(KeyIdx);
			NewKey.Y = EdInterface->EvalSub(SubIdx, NewKey.X);

			FIntPoint NewKeyPos = CalcScreenPos(NewKey);

			// If this section is visible then draw it!
			bool bSectionVisible = true;
			if(NewKey.X < SharedData->StartIn || OldKey.X > SharedData->EndIn)
			{
				bSectionVisible = false;
			}

			if(KeyIdx>0 && bSectionVisible)
			{
				float KeyDiff = NewKey.X - OldKey.X;
				// We need to take the total range into account... 
				// otherwise, we end up w/ 100,000s of steps.
				float Scalar = 1.0f;
				float KeyDiffTemp = KeyDiff;
				while (FMath::TruncToInt(KeyDiffTemp / Scalar) > 1)
				{
					Scalar *= 10.0f;
				}
				float DrawTrackInRes = CurveDrawRes/PixelsPerIn;
				DrawTrackInRes *= Scalar;
				int32 NumSteps = FMath::CeilToInt(KeyDiff/DrawTrackInRes);

				if(Scalar > 1.0f)
				{
					const int32 MinStepsToConsider = 30;
					if(NumSteps < MinStepsToConsider)
					{
						// Make sure at least some steps are drawn.  The scalar might have made it so that only 1 step is drawn
						NumSteps = MinStepsToConsider;
					}
				}

				float DrawSubstep = KeyDiff/NumSteps;

				// Find position on first keyframe.
				FVector2D Old = OldKey;
				FIntPoint OldPos = OldKeyPos;

				EInterpCurveMode InterpMode = EdInterface->GetKeyInterpMode(KeyIdx-1);

				LineItem.SetColor( Entry.CurveColor );
				if(Canvas->IsHitTesting()) Canvas->SetHitProxy(new HCurveEditorLineProxy(CurveIndex, SubIdx));
				// For constant interpolation - don't draw ticks - just draw dotted line.
				if(InterpMode == CIM_Constant)
				{
					LineItem.Draw( Canvas,  OldKeyPos, FVector2D(NewKeyPos.X, OldKeyPos.Y) );
					LineItem.Draw( Canvas,  FVector2D(NewKeyPos.X, OldKeyPos.Y), NewKeyPos );
				}
				else if(InterpMode == CIM_Linear && !Entry.bColorCurve)
				{
					LineItem.Draw( Canvas, OldKeyPos, NewKeyPos );
				}
				else
				{
					// Then draw a line for each substep.
					for(int32 j=1; j<NumSteps+1; j++)
					{
						FVector2D New;
						New.X = OldKey.X + j*DrawSubstep;
						New.Y = EdInterface->EvalSub(SubIdx, New.X);

						FColor StepColor = Entry.bColorCurve ? GetLineColor(EdInterface, New.X, !!Entry.bFloatingPointColorCurve) : Entry.CurveColor;

						FIntPoint NewPos = CalcScreenPos(New);
						
						LineItem.SetColor( StepColor );
						LineItem.Draw( Canvas,  OldPos, NewPos );
						
						Old = New;
						OldPos = NewPos;
					}
				}

				if(Canvas->IsHitTesting()) Canvas->SetHitProxy(NULL);
			}

			OldKey = NewKey;
			OldKeyPos = NewKeyPos;
		}

		// Draw lines to continue curve beyond last and before first.
		if(Canvas->IsHitTesting()) Canvas->SetHitProxy(new HCurveEditorLineProxy(CurveIndex, SubIdx));

		if(NumKeys > 0)
		{
			float RangeStart, RangeEnd;
			EdInterface->GetInRange(RangeStart, RangeEnd);

			if(RangeStart > SharedData->StartIn)
			{
				FVector2D FirstKey;
				FirstKey.X = RangeStart;
				FirstKey.Y = EdInterface->GetKeyOut(SubIdx, 0);

				FColor StepColor = Entry.bColorCurve ? GetLineColor(EdInterface, RangeStart, !!Entry.bFloatingPointColorCurve) : Entry.CurveColor;
				FIntPoint FirstKeyPos = CalcScreenPos(FirstKey);
				
				LineItem.SetColor( StepColor );
				LineItem.Draw( Canvas, FVector2D(LabelWidth, FirstKeyPos.Y), FirstKeyPos );
				
			}

			if(RangeEnd < SharedData->EndIn)
			{
				FVector2D LastKey;
				LastKey.X = RangeEnd;
				LastKey.Y = EdInterface->GetKeyOut(SubIdx, NumKeys-1);;

				FColor StepColor = Entry.bColorCurve ? GetLineColor(EdInterface, RangeEnd, !!Entry.bFloatingPointColorCurve) : Entry.CurveColor;
				FIntPoint LastKeyPos = CalcScreenPos(LastKey);

				LineItem.SetColor( StepColor );
				LineItem.Draw( Canvas, LastKeyPos, FVector2D(LabelWidth+CurveViewX, LastKeyPos.Y) );				
			}
		}
		else // No points - draw line at zero.
		{
			FIntPoint OriginPos = CalcScreenPos(FVector2D::ZeroVector);
			LineItem.SetColor( Entry.CurveColor );
			LineItem.Draw( Canvas, FVector2D(LabelWidth, OriginPos.Y), FVector2D(LabelWidth+CurveViewX, OriginPos.Y) );			
		}

		if(Canvas->IsHitTesting()) Canvas->SetHitProxy(NULL);

		// Draw keypoints on top of curve
		for(int32 KeyIdx=0; KeyIdx<NumKeys; KeyIdx++)
		{
			FVector2D NewKey;
			NewKey.X = EdInterface->GetKeyIn(KeyIdx);
			NewKey.Y = EdInterface->GetKeyOut(SubIdx, KeyIdx);

			FIntPoint NewKeyPos = CalcScreenPos(NewKey);

			FCurveEditorSelectedKey TestKey(CurveIndex, SubIdx, KeyIdx);
			bool bSelectedKey = SharedData->SelectedKeys.Contains(TestKey);
			FColor BorderColor = EdInterface->GetKeyColor(SubIdx, KeyIdx, Entry.CurveColor);
			FColor CenterColor = bSelectedKey ? SelectedKeyColor.ToFColor(true) : Entry.CurveColor;

			if(Canvas->IsHitTesting()) Canvas->SetHitProxy(new HCurveEditorKeyProxy(CurveIndex, SubIdx, KeyIdx));
			Canvas->DrawTile( NewKeyPos.X-3, NewKeyPos.Y-3, 7, 7, 0.f, 0.f, 1.f, 1.f, BorderColor);
			Canvas->DrawTile( NewKeyPos.X-2, NewKeyPos.Y-2, 5, 5, 0.f, 0.f, 1.f, 1.f, CenterColor);
			if(Canvas->IsHitTesting()) Canvas->SetHitProxy(NULL);

			// If previous section is a curve- show little handles.
			if(bSelectedKey || SharedData->bShowAllCurveTangents)
			{
				float ArriveTangent, LeaveTangent;
				EdInterface->GetTangents(SubIdx, KeyIdx, ArriveTangent, LeaveTangent);	

				EInterpCurveMode PrevMode = (KeyIdx > 0) ? EdInterface->GetKeyInterpMode(KeyIdx-1) : EInterpCurveMode(255);
				EInterpCurveMode NextMode = (KeyIdx < NumKeys-1) ? EdInterface->GetKeyInterpMode(KeyIdx) : EInterpCurveMode(255);

				// If not first point, and previous mode was a curve type.
				if(PrevMode == CIM_CurveAuto || PrevMode == CIM_CurveAutoClamped || PrevMode == CIM_CurveUser || PrevMode == CIM_CurveBreak)
				{
					FVector2D HandleDir = CalcTangentDir((PixelsPerOut/PixelsPerIn) * ArriveTangent);

					FIntPoint HandlePos;
					HandlePos.X = NewKeyPos.X - FMath::RoundToInt(HandleDir.X * HandleLength);
					HandlePos.Y = NewKeyPos.Y - FMath::RoundToInt(HandleDir.Y * HandleLength);
					LineItem.SetColor( FLinearColor::White );
					LineItem.Draw( Canvas, NewKeyPos, HandlePos );
					

					if(Canvas->IsHitTesting()) Canvas->SetHitProxy(new HCurveEditorKeyHandleProxy(CurveIndex, SubIdx, KeyIdx, true));
					Canvas->DrawTile(HandlePos.X - 2, HandlePos.Y - 2, 5, 5, 0.f, 0.f, 1.f, 1.f, FColor::White);
					if(Canvas->IsHitTesting()) Canvas->SetHitProxy(NULL);
				}

				// If next section is a curve, draw leaving handle.
				if(NextMode == CIM_CurveAuto || NextMode == CIM_CurveAutoClamped || NextMode == CIM_CurveUser || NextMode == CIM_CurveBreak)
				{
					FVector2D HandleDir = CalcTangentDir((PixelsPerOut/PixelsPerIn) * LeaveTangent);

					FIntPoint HandlePos;
					HandlePos.X = NewKeyPos.X + FMath::RoundToInt(HandleDir.X * HandleLength);
					HandlePos.Y = NewKeyPos.Y + FMath::RoundToInt(HandleDir.Y * HandleLength);

					LineItem.SetColor( FLinearColor::White );
					LineItem.Draw( Canvas, NewKeyPos, HandlePos );
					
					if(Canvas->IsHitTesting()) Canvas->SetHitProxy(new HCurveEditorKeyHandleProxy(CurveIndex, SubIdx, KeyIdx, false));
					Canvas->DrawTile(HandlePos.X - 2, HandlePos.Y - 2, 5, 5, 0.f, 0.f, 1.f, 1.f, FColor::White);
					if(Canvas->IsHitTesting()) Canvas->SetHitProxy(NULL);
				}
			}

			// If mouse is over this keypoint, show its value
			if(CurveIndex == MouseOverCurveIndex &&
				SubIdx == MouseOverSubIndex &&
				KeyIdx == MouseOverKeyIndex)
			{
				FString KeyComment;
				if (bSnapToFrames)
				{
					KeyComment = FString::Printf(TEXT("(%" INT64_FMT "f,%3.2f)"), FMath::RoundToInt(NewKey.X/InSnapAmount), NewKey.Y);
				}
				else
				{
					KeyComment = FString::Printf(TEXT("(%3.2f,%3.2f)"), NewKey.X, NewKey.Y);
				}

				FCanvasTextItem TextItem( FVector2D(NewKeyPos.X + 5, NewKeyPos.Y - 5), FText::FromString(KeyComment), GEditor->GetSmallFont(), GridTextColor );
				Canvas->DrawItem( TextItem );				
			}
		}
	}
}

namespace CurveEditor
{
	float GetGridSpacing(int32 GridNum)
	{
		if(GridNum & 0x01) // Odd numbers
		{
			return FMath::Pow( 10.f, 0.5f*((float)(GridNum-1)) + 1.f );
		}
		else // Even numbers
		{
			return 0.5f * FMath::Pow( 10.f, 0.5f*((float)(GridNum)) + 1.f );
		}
	}

	/** Calculate the best frames' density. */
	uint32 CalculateBestFrameStep(float SnapAmount, float PixelsPerSec, float MinPixelsPerGrid)
	{
		uint32 FrameRate = FMath::CeilToInt(1.0f / SnapAmount);
		uint32 FrameStep = 1;

		// Calculate minimal-symmetric integer divisor.
		uint32 MinFrameStep = FrameRate;
		uint32 i = 2;	
		while ( i < MinFrameStep )
		{
			if ( MinFrameStep % i == 0 )
			{
				MinFrameStep /= i;
				i = 1;
			}
			i++;
		}	

		// Find the best frame step for certain grid density.
		while (	FrameStep * SnapAmount * PixelsPerSec < MinPixelsPerGrid )
		{
			FrameStep++;
			if ( FrameStep < FrameRate )
			{
				// Must be divisible by MinFrameStep and divisor of FrameRate.
				while ( !(FrameStep % MinFrameStep == 0 && FrameRate % FrameStep == 0) )
				{
					FrameStep++;
				}
			}
			else
			{
				// Must be multiple of FrameRate.
				while ( FrameStep % FrameRate != 0 )
				{
					FrameStep++;
				}
			}
		}

		return FrameStep;
	}
}

void FCurveEditorViewportClient::DrawGrid(FViewport* Viewport, FCanvas* Canvas)
{
	// Determine spacing for In and Out grid lines
	int32 MinPixelsPerInGrid = 35;
	int32 MinPixelsPerOutGrid = 25;

	float MinGridSpacing = 0.001f;
	int32 GridNum = 0;

	float InGridSpacing = MinGridSpacing;
	while(InGridSpacing * PixelsPerIn < MinPixelsPerInGrid)
	{
		InGridSpacing = MinGridSpacing * CurveEditor::GetGridSpacing(GridNum);
		GridNum++;
	}

	GridNum = 0;

	float OutGridSpacing = MinGridSpacing;
	while(OutGridSpacing * PixelsPerOut < MinPixelsPerOutGrid)
	{
		OutGridSpacing = MinGridSpacing * CurveEditor::GetGridSpacing(GridNum);
		GridNum++;
	}

	int32 XL, YL;
	StringSize(GEngine->GetSmallFont(), XL, YL, TEXT("0123456789"));


	// Calculate best frames' step.
	uint32 FrameStep = 1; // Important frames' density.
	uint32 AuxFrameStep = 1; // Auxiliary frames' density.

	if (bSnapToFrames)
	{
		InGridSpacing  = InSnapAmount;	
		FrameStep = CurveEditor::CalculateBestFrameStep(InSnapAmount, PixelsPerIn, MinPixelsPerInGrid);
		AuxFrameStep = CurveEditor::CalculateBestFrameStep(InSnapAmount, PixelsPerIn, 6);
	}

	const static FLinearColor NormalLine(FColor(80, 80, 80));
	const static FLinearColor ImportantLine(FColor(110, 110, 110));

	// Draw input grid
	FCanvasLineItem LineItem;
	int32 InNum = FMath::FloorToInt(SharedData->StartIn/InGridSpacing);
	while(InNum * InGridSpacing < SharedData->EndIn)
	{
		FLinearColor LineColor = GridColor;

		// Change line color for important frames.
		if (bSnapToFrames)
		{
			LineColor = NormalLine;
			if (InNum % FrameStep == 0)
			{
				LineColor = ImportantLine;
			}
		}

		// Draw grid line.
		// In frames mode auxiliary lines cannot be too close.
		FIntPoint GridPos = CalcScreenPos(FVector2D(InNum * InGridSpacing, 0.f));		
		if (!bSnapToFrames || (FMath::Abs(InNum) % AuxFrameStep == 0))
		{
			LineItem.SetColor( LineColor );
			LineItem.Draw( Canvas, FVector2D(GridPos.X, 0), FVector2D(GridPos.X, CurveViewY) );
		}
		InNum++;
	}

	// Draw output grid
	int32 OutNum = FMath::FloorToInt(SharedData->StartOut/OutGridSpacing);
	while(OutNum*OutGridSpacing < SharedData->EndOut)
	{
		FIntPoint GridPos = CalcScreenPos(FVector2D(0.f, OutNum*OutGridSpacing));
		LineItem.SetColor( GridColor );
		LineItem.Draw( Canvas, FVector2D(LabelWidth, GridPos.Y), FVector2D(LabelWidth + CurveViewX, GridPos.Y) );
		OutNum++;
	}

	// Calculate screen position of graph origin and draw white lines to indicate it

	FIntPoint OriginPos = CalcScreenPos(FVector2D::ZeroVector);

	LineItem.Draw( Canvas, FVector2D(LabelWidth, OriginPos.Y), FVector2D(LabelWidth+CurveViewX, OriginPos.Y) );
	LineItem.Draw( Canvas, FVector2D(OriginPos.X, 0), FVector2D(OriginPos.X, CurveViewY) );
	
	// Draw input labels
	FCanvasTextItem TextItem( FVector2D::ZeroVector, FText::GetEmpty(), GEditor->GetSmallFont(), GridTextColor );
	
	InNum = FMath::FloorToInt(SharedData->StartIn/InGridSpacing);
	while(InNum*InGridSpacing < SharedData->EndIn)
	{
		// Draw value label
		FIntPoint GridPos = CalcScreenPos(FVector2D(InNum*InGridSpacing, 0.f));

		// Show time or important frames' numbers (based on FrameStep).
		if (!bSnapToFrames || FMath::Abs(InNum) % FrameStep == 0)
		{				
			FString Label;
			if (bSnapToFrames)
			{
				// Show frames' numbers.
				Label = FString::Printf(TEXT("%d"), InNum);
			}
			else
			{
				// Show time.
				Label = FString::Printf(TEXT("%3.2f"), InNum*InGridSpacing);
			}
			TextItem.Text = FText::FromString( Label );
			Canvas->DrawItem( TextItem, GridPos.X + 2, CurveViewY - YL - 2 );				
		}
		InNum++;
	}

	// Draw output labels

	OutNum = FMath::FloorToInt(SharedData->StartOut/OutGridSpacing);
	while(OutNum*OutGridSpacing < SharedData->EndOut)
	{
		FIntPoint GridPos = CalcScreenPos(FVector2D(0.f, OutNum*OutGridSpacing));
		if(GridPos.Y < CurveViewY - YL) // Only draw Output scale numbering if its not going to be on top of input numbering.
		{
			FString ScaleLabel = FString::Printf(TEXT("%3.2f"), OutNum*OutGridSpacing);
			TextItem.Text = FText::FromString( ScaleLabel );
			Canvas->DrawItem( TextItem, LabelWidth + 2, GridPos.Y - YL - 2 );
		}
		OutNum++;
	}
}

FIntPoint FCurveEditorViewportClient::CalcScreenPos(const FVector2D& Val)
{
	FIntPoint Result;
	Result.X = LabelWidth + ((Val.X - SharedData->StartIn) * PixelsPerIn);
	Result.Y = CurveViewY - ((Val.Y - SharedData->StartOut) * PixelsPerOut);
	return Result;
}

FVector2D FCurveEditorViewportClient::CalcValuePoint(const FVector2D& Pos)
{
	FVector2D Result;
	Result.X = SharedData->StartIn + ((Pos.X - LabelWidth) / PixelsPerIn);
	Result.Y = SharedData->StartOut + ((CurveViewY - Pos.Y) / PixelsPerOut);
	return Result;
}

FColor FCurveEditorViewportClient::GetLineColor(FCurveEdInterface* EdInterface, float InVal, bool bFloatingPointColor)
{
	FColor StepColor;

	int32 NumSubs = EdInterface->GetNumSubCurves();
	if(NumSubs == 3)
	{
		if (bFloatingPointColor == true)
		{
			float Value;

			Value = EdInterface->EvalSub(0, InVal);
			Value *= 255.f;
			StepColor.R = FMath::RoundToInt(FMath::Clamp<float>(Value, 0.f, 255.f));
			Value = EdInterface->EvalSub(1, InVal);
			Value *= 255.f;
			StepColor.G = FMath::RoundToInt(FMath::Clamp<float>(Value, 0.f, 255.f));
			Value = EdInterface->EvalSub(2, InVal);
			Value *= 255.f;
			StepColor.B = FMath::RoundToInt(FMath::Clamp<float>(Value, 0.f, 255.f));
		}
		else
		{
			StepColor.R = FMath::RoundToInt(FMath::Clamp<float>(EdInterface->EvalSub(0, InVal), 0.f, 255.f));
			StepColor.G = FMath::RoundToInt(FMath::Clamp<float>(EdInterface->EvalSub(1, InVal), 0.f, 255.f));
			StepColor.B = FMath::RoundToInt(FMath::Clamp<float>(EdInterface->EvalSub(2, InVal), 0.f, 255.f));
		}
		StepColor.A = 255;
	}
	else if(NumSubs == 1)
	{
		if (bFloatingPointColor == true)
		{
			float Value;

			Value = EdInterface->EvalSub(0, InVal);
			Value *= 255.f;
			StepColor.R = FMath::RoundToInt(FMath::Clamp<float>(Value, 0.f, 255.f));
		}
		else
		{
			StepColor.R = FMath::RoundToInt(FMath::Clamp<float>(EdInterface->EvalSub(0, InVal), 0.f, 255.f));
		}
		StepColor.G = StepColor.R;
		StepColor.B = StepColor.R;
		StepColor.A = 255;
	}
	else
	{
		StepColor = FColor(0,0,0);
	}

	return StepColor;
}

FVector2D FCurveEditorViewportClient::CalcTangentDir(float Tangent)
{
	const float Angle = FMath::Atan(Tangent);
	return FVector2D(FMath::Cos(Angle), -FMath::Sin(Angle));
}

float FCurveEditorViewportClient::CalcTangent(const FVector2D& HandleDelta)
{
	// Ensure X is positive and non-zero.
	// Tangent is gradient of handle.
	return HandleDelta.Y / FMath::Max<double>(HandleDelta.X, KINDA_SMALL_NUMBER);
}

int32 FCurveEditorViewportClient::AddNewKeypoint(int32 InCurveIndex, int32 InSubIndex, const FIntPoint& ScreenPos)
{
	check(InCurveIndex >= 0 && InCurveIndex < SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num());

	FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[InCurveIndex];
	FVector2D NewKeyVal = CalcValuePoint(FVector2D(ScreenPos) / GetDPIScale());
	float NewKeyIn = SnapIn(NewKeyVal.X);

	FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

	int32 NewKeyIndex = INDEX_NONE;
	if(EdInterface)
	{
		// Notify a containing tool etc. before and after we add the new key.
		if(SharedData->NotifyObject)
		{
			TArray<UObject*> CurvesAboutToChange;
			CurvesAboutToChange.Add(Entry.CurveObject);
			SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);
		}

		NewKeyIndex = EdInterface->CreateNewKey(NewKeyIn);
		EdInterface->SetKeyInterpMode(NewKeyIndex, CIM_CurveAutoClamped);

		if(SharedData->NotifyObject)
		{
			SharedData->NotifyObject->PostEditCurve();
		}
	}
	return NewKeyIndex;
}

void FCurveEditorViewportClient::MoveSelectedKeys(float DeltaIn, float DeltaOut)
{
	// To avoid applying an input-modify twice to the same key (but on different subs), we note which 
	// curve/key combination we have already change the In of.
	TArray<FCurveEditorModKey> MovedInKeys;

	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[i];

		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

		// If there is a change in the Output, apply it.
		if(FMath::Abs<float>(DeltaOut) > 0.f)
		{
			SelKey.UnsnappedOut += DeltaOut;
			float NewOut = SelKey.UnsnappedOut;

			// For colour curves, clamp keys to between 0 and 255(ish)
			if(Entry.bColorCurve && (Entry.bFloatingPointColorCurve == false))
			{
				NewOut = FMath::Clamp<float>(NewOut, 0.f, 255.9f);
			}
			if (Entry.bClamp)
			{
				NewOut = FMath::Clamp<float>(NewOut, Entry.ClampLow, Entry.ClampHigh);
			}
			EdInterface->SetKeyOut(SelKey.SubIndex, SelKey.KeyIndex, NewOut);
		}

		FCurveEditorModKey KeyToTest(SelKey.CurveIndex, SelKey.KeyIndex);

		// If there is a change in the Input, apply it.
		// This is slightly complicated because it may change the index of the selected key, so we have to update the selection as we do it.
		if(FMath::Abs<float>(DeltaIn) > 0.f && !MovedInKeys.Contains(KeyToTest))
		{
			SelKey.UnsnappedIn += DeltaIn;
			float NewIn = SnapIn(SelKey.UnsnappedIn);

			int32 OldKeyIndex = SelKey.KeyIndex;
			int32 NewKeyIndex = EdInterface->SetKeyIn(SelKey.KeyIndex, NewIn);
			SelKey.KeyIndex = NewKeyIndex;

			// If the key changed index we need to search for any other selected keys on this track that may need their index adjusted because of this change.
			int32 KeyMove = NewKeyIndex - OldKeyIndex;
			if(KeyMove > 0)
			{
				for(int32 j=0; j<SharedData->SelectedKeys.Num(); j++)
				{
					if(j == i) // Don't look at one we just changed.
						continue;

					FCurveEditorSelectedKey& TestKey = SharedData->SelectedKeys[j];
					if(TestKey.CurveIndex == SelKey.CurveIndex && 
						TestKey.KeyIndex > OldKeyIndex && 
						TestKey.KeyIndex <= NewKeyIndex)
					{
						TestKey.KeyIndex--;
					}
					// change index of subcurves also
					else if (TestKey.CurveIndex == SelKey.CurveIndex && 
						TestKey.KeyIndex == OldKeyIndex)
					{
						check(TestKey.SubIndex != SelKey.SubIndex);
						TestKey.KeyIndex = NewKeyIndex;
					}
				}
			}
			else if(KeyMove < 0)
			{
				for(int32 j=0; j<SharedData->SelectedKeys.Num(); j++)
				{
					if(j == i)
						continue;

					FCurveEditorSelectedKey& TestKey = SharedData->SelectedKeys[j];
					if(TestKey.CurveIndex == SelKey.CurveIndex && 
						TestKey.KeyIndex < OldKeyIndex && 
						TestKey.KeyIndex >= NewKeyIndex)
					{
						TestKey.KeyIndex++;
					}
					// change index of subcurves also
					else if (TestKey.CurveIndex == SelKey.CurveIndex && 
						TestKey.KeyIndex == OldKeyIndex)
					{
						check(TestKey.SubIndex != SelKey.SubIndex);
						TestKey.KeyIndex = NewKeyIndex;
					}
				}
			}

			// Remember we have adjusted the In of this key.
			KeyToTest.KeyIndex = NewKeyIndex;
			MovedInKeys.Add(KeyToTest);
		}
	} // FOR each selected key

	// Call the notify object if present.
	if(SharedData->NotifyObject)
	{
		SharedData->NotifyObject->MovedKey();
	}
}

void FCurveEditorViewportClient::MoveCurveHandle(const FVector2D& NewHandleVal)
{
	FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[HandleCurveIndex];
	FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
	check(EdInterface);

	FVector2D KeyVal;
	KeyVal.X = EdInterface->GetKeyIn(HandleKeyIndex);
	KeyVal.Y = EdInterface->GetKeyOut(HandleSubIndex, HandleKeyIndex);

	// Find vector (in 'curve space') between key point and mouse position.
	FVector2D HandleDelta = NewHandleVal - KeyVal;

	// If 'arriving' handle (at end of section), the handle points the other way.
	if(bHandleArriving)
	{
		HandleDelta *= -1.f;
	}

	float NewTangent = CalcTangent(HandleDelta);

	float ArriveTangent, LeaveTangent;
	EdInterface->GetTangents(HandleSubIndex, HandleKeyIndex, ArriveTangent, LeaveTangent);

	// If adjusting the handle on an 'auto' keypoint, automagically convert to User mode.
	EInterpCurveMode InterpMode = EdInterface->GetKeyInterpMode(HandleKeyIndex);
	if(InterpMode == CIM_CurveAuto || InterpMode == CIM_CurveAutoClamped)
	{
		EdInterface->SetKeyInterpMode(HandleKeyIndex, CIM_CurveUser);
	}

	// In both User and Auto (non-Break curve modes) - enforce smoothness.
	if(InterpMode != CIM_CurveBreak)
	{
		ArriveTangent = NewTangent;
		LeaveTangent = NewTangent;
	}
	else
	{
		if(bHandleArriving)
		{
			ArriveTangent = NewTangent;
		}
		else
		{
			LeaveTangent = NewTangent;
		}
	}

	EdInterface->SetTangents(HandleSubIndex, HandleKeyIndex, ArriveTangent, LeaveTangent);
}

void FCurveEditorViewportClient::AddKeyToSelection(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex)
{
	if(!KeyIsInSelection(InCurveIndex, InSubIndex, InKeyIndex))
	{
		SharedData->SelectedKeys.Add(FCurveEditorSelectedKey(InCurveIndex, InSubIndex, InKeyIndex));
	}
}

void FCurveEditorViewportClient::RemoveKeyFromSelection(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex)
{
	FCurveEditorSelectedKey TestKey(InCurveIndex, InSubIndex, InKeyIndex);

	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		if(SharedData->SelectedKeys[i] == TestKey)
		{
			SharedData->SelectedKeys.RemoveAt(i);
			return;
		}
	}
}

bool FCurveEditorViewportClient::KeyIsInSelection(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex)
{
	FCurveEditorSelectedKey TestKey(InCurveIndex, InSubIndex, InKeyIndex);

	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		if(SharedData->SelectedKeys[i] == TestKey)
		{
			return true;
		}
	}

	return false;
}

void FCurveEditorViewportClient::SetInSnap(bool bEnabled, float SnapAmount, bool bInSnapToFrames)
{
	bSnapEnabled = bEnabled;
	InSnapAmount = SnapAmount;
	bSnapToFrames = bInSnapToFrames;
}

float FCurveEditorViewportClient::UpdateViewportClientWindowDPIScale() const
{
	float DPIScale = 1.f;
	if (CurveEditorViewportPtr.IsValid())
	{
		TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(CurveEditorViewportPtr.Pin().ToSharedRef());
		if (WidgetWindow.IsValid())
		{
			DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
		}
	}

	return DPIScale;
}

float FCurveEditorViewportClient::SnapIn(float InValue)
{
	if(bSnapEnabled)
	{
		return InSnapAmount * FMath::RoundToFloat(InValue/InSnapAmount);
	}
	else
	{
		return InValue;
	}
}

void FCurveEditorViewportClient::BeginMoveSelectedKeys()
{
	TArray<UObject*> CurvesAboutToChange;

	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[i];

		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
		check(EdInterface);

		SharedData->SelectedKeys[i].UnsnappedIn = EdInterface->GetKeyIn(SelKey.KeyIndex);
		SharedData->SelectedKeys[i].UnsnappedOut = EdInterface->GetKeyOut(SelKey.SubIndex, SelKey.KeyIndex);

		// Make a list of all curves we are going to move keys in.
		if(Entry.CurveObject)
		{
			CurvesAboutToChange.AddUnique(Entry.CurveObject);
		}
	}

	if(SharedData->NotifyObject)
	{
		SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);
	}
}

void FCurveEditorViewportClient::EndMoveSelectedKeys()
{
	if(SharedData->NotifyObject)
	{
		SharedData->NotifyObject->PostEditCurve();
	}
}

void FCurveEditorViewportClient::ToggleCurveHidden(int32 InCurveIndex)
{
	FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[InCurveIndex];
	CURVEEDENTRY_TOGGLE_HIDECURVE(Entry.bHideCurve);

	// Remove any key we have selected in the current curve.
	int32 i=0;
	while(i<SharedData->SelectedKeys.Num())
	{
		if(SharedData->SelectedKeys[i].CurveIndex == InCurveIndex)
		{
			SharedData->SelectedKeys.RemoveAt(i);
		}
		else
		{
			i++;
		}
	}
}

void FCurveEditorViewportClient::ToggleSubCurveHidden(int32 InCurveIndex, int32 InSubCurveIndex)
{
	FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[InCurveIndex];
	check(InSubCurveIndex < 6);
	CURVEEDENTRY_TOGGLE_HIDESUBCURVE(Entry.bHideCurve, InSubCurveIndex);
}

void FCurveEditorViewportClient::HandleWindowDPIScaleChanged(TSharedRef<SWindow> InWindow)
{
	RequestUpdateDPIScale();
	if (CurveEditorViewportPtr.IsValid())
	{
		CurveEditorViewportPtr.Pin()->RefreshViewport();
	}
}
