// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "InputCoreTypes.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UnrealClient.h"
#include "ViewportClient.h"

class FCanvas;
class FCurveEdInterface;
class FCurveEditorSharedData;
class HHitProxy;
class SCurveEditorViewport;
class SDistributionCurveEditor;
class SWindow;
struct FCurveEdEntry;

/*-----------------------------------------------------------------------------
   FCurveEditorViewportClient
-----------------------------------------------------------------------------*/

class FCurveEditorViewportClient : public FCommonViewportClient
{
public:
	/** Constructor */
	FCurveEditorViewportClient(TWeakPtr<SDistributionCurveEditor> InCurveEditor, TWeakPtr<SCurveEditorViewport> InCurveEditorViewport);
	virtual ~FCurveEditorViewportClient() override;

	/** FViewportClient interface */
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual void MouseMove(FViewport* Viewport, int32 X, int32 Y) override;
	virtual void CapturedMouseMove(FViewport* Viewport, int32 X, int32 Y) override;
	virtual bool InputAxis(const FInputKeyEventArgs& Args) override;

	/** Exec handler */
	void Exec(const TCHAR* Cmd);

	/** Returns the ratio of the size of the particle emitters to the size of the viewport */
	float GetViewportVerticalScrollBarRatio() const;

	/** Set snap behavior */
	void SetInSnap(bool bEnabled, float SnapAmount, bool bInSnapToFrames);

	/** FViewport interface */
	virtual float UpdateViewportClientWindowDPIScale() const override;
private:
	/** Curve editor key movement axis locking */
	enum ECurveEdMovementAxisLock
	{
		AxisLock_None,
		AxisLock_Horizontal,
		AxisLock_Vertical
	};

	/** Checks to see if the hit was on a non-graph element and processes */
	bool ProcessNonGraphHit(HHitProxy* HitResult);

	/** Updates the states of the scrollbars */
	void UpdateScrollBars();

	/** Changes the position of the vertical scrollbar (on a mouse scrollwheel event) */
	void ChangeViewportScrollBarPosition(EScrollDirection Direction);

	/** Returns the positions of the scrollbars relative to the Particle Particles */
	FVector2D GetViewportScrollBarPositions() const;

	/** Drawing utils */
	void DrawEntry(FViewport* Viewport, FCanvas* Canvas, const FCurveEdEntry& Entry, int32 CurveIndex);
	void DrawGrid(FViewport* Viewport, FCanvas* Canvas);

	/** Helper methods */
	FIntPoint CalcScreenPos(const FVector2D& Val);
	FVector2D CalcValuePoint(const FVector2D& Pos);
	FColor GetLineColor(FCurveEdInterface* EdInterface, float InVal, bool bFloatingPointColor);
	FVector2D CalcTangentDir(float Tangent);
	float CalcTangent(const FVector2D& HandleDelta);
	float SnapIn(float InValue);

	/** Adds a new key */
	int32 AddNewKeypoint(int32 InCurveIndex, int32 InSubIndex, const FIntPoint& ScreenPos);
	
	/** Add/remove a key to/from the selection */
	void AddKeyToSelection(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex);
	void RemoveKeyFromSelection(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex);
	
	/** Returns true if the specified key is currently selected */
	bool KeyIsInSelection(int32 InCurveIndex, int32 InSubIndex, int32 InKeyIndex);

	/** Handles keys being moved by the user */
	void BeginMoveSelectedKeys();
	void MoveSelectedKeys(float DeltaIn, float DeltaOut);
	void EndMoveSelectedKeys();

	/** Handles keys being moved by the user */
	void MoveCurveHandle(const FVector2D& NewHandleVal);

	/** Hide/show curves */
	void ToggleCurveHidden(int32 InCurveIndex);
	void ToggleSubCurveHidden(int32 InCurveIndex, int32 InSubCurveIndex);

	void HandleWindowDPIScaleChanged(TSharedRef<SWindow> Window);

private:
	/** Pointer back to the Particle editor tool that owns us */
	TWeakPtr<SDistributionCurveEditor> CurveEditorPtr;

	/** Pointer back to the Particle viewport control that owns us */
	TWeakPtr<SCurveEditorViewport> CurveEditorViewportPtr;
	
	/** Data and methods shared across multiple classes */
	TSharedPtr<FCurveEditorSharedData> SharedData;

	FIntPoint LabelOrigin2D;

	const int32	LabelWidth;
	const int32	ColorKeyWidth;
	const float ZoomSpeed;
	const float MouseZoomSpeed;
	const float HandleLength;
	const float FitMargin;
	const int32 CurveDrawRes;

	int32 DragStartMouseX;
	int32 DragStartMouseY;
	int32 OldMouseX;
	int32 OldMouseY;
	bool bPanning;
	bool bMouseDown;
	bool bDraggingHandle;
	bool bBegunMoving;
	ECurveEdMovementAxisLock MovementAxisLock;
	bool bBoxSelecting;
	bool bKeyAdded;
	int32 DistanceDragged;
	int32 BoxStartX;
	int32 BoxStartY;
	int32 BoxEndX;
	int32 BoxEndY;

	float CurveViewX;
	float CurveViewY;
	float PixelsPerIn;
	float PixelsPerOut;
	FLinearColor BackgroundColor;
	FLinearColor LabelColor;
	FLinearColor SelectedLabelColor;
	FLinearColor GridColor;
	FLinearColor GridTextColor;
	FLinearColor LabelBlockBkgColor;
	FLinearColor SelectedKeyColor;

	int32 MouseOverCurveIndex;
	int32 MouseOverSubIndex;
	int32 MouseOverKeyIndex;

	int32 HandleCurveIndex;
	int32 HandleSubIndex;
	int32 HandleKeyIndex;
	bool bHandleArriving;

	bool bSnapEnabled;
	float InSnapAmount;
	bool bSnapToFrames;
};
