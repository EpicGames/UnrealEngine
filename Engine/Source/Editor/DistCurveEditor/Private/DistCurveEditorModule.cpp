// Copyright Epic Games, Inc. All Rights Reserved.

#include "DistCurveEditorModule.h"

#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "SDistributionCurveEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

const FName DistCurveEditorAppIdentifier = FName(TEXT("DistCurveEditorApp"));


/*-----------------------------------------------------------------------------
   FDistributionCurveEditorModule
-----------------------------------------------------------------------------*/

class FDistributionCurveEditorModule : public IDistributionCurveEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FDistributionCurveEditorModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
	}

	virtual TSharedRef<IDistributionCurveEditor> CreateCurveEditorWidget(UInterpCurveEdSetup* EdSetup, FCurveEdNotifyInterface* NotifyObject) override
	{
		return CreateCurveEditorWidget(EdSetup, NotifyObject, IDistributionCurveEditor::FCurveEdOptions());
	}

	virtual TSharedRef<IDistributionCurveEditor> CreateCurveEditorWidget(UInterpCurveEdSetup* EdSetup, FCurveEdNotifyInterface* NotifyObject, IDistributionCurveEditor::FCurveEdOptions Options) override
	{
		TSharedRef<IDistributionCurveEditor> Widget = SNew(SDistributionCurveEditor).EdSetup(EdSetup).NotifyObject(NotifyObject).CurveEdOptions(Options);
		return Widget;
	}
};

IMPLEMENT_MODULE( FDistributionCurveEditorModule, DistCurveEditor );
