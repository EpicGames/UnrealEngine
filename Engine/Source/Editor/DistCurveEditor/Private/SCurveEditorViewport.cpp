// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveEditorViewport.h"

#include "CurveEditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Slate/SceneViewport.h"
#include "SlotBase.h"
#include "ViewportClient.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"


SCurveEditorViewport::~SCurveEditorViewport()
{
}

void SCurveEditorViewport::Construct(const FArguments& InArgs)
{
	CurveEditorPtr = InArgs._CurveEditor;
	
	bool bAlwaysShowScrollbar = InArgs._CurveEdOptions.bAlwaysShowScrollbar;

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				SAssignNew(ViewportWidget, SViewport)
				.EnableGammaCorrection(false)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				.ShowEffectWhenDisabled(false)
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			bAlwaysShowScrollbar ? 
			( 
				SAssignNew(ViewportVerticalScrollBar, SScrollBar)
				.AlwaysShowScrollbar(true)
				.OnUserScrolled(this, &SCurveEditorViewport::OnViewportVerticalScrollBarScrolled)
			) :
			(
				SAssignNew(ViewportVerticalScrollBar, SScrollBar)
				.Visibility(this, &SCurveEditorViewport::GetViewportVerticalScrollBarVisibility)
				.OnUserScrolled(this, &SCurveEditorViewport::OnViewportVerticalScrollBarScrolled)
			)
		]
	];
	
	ViewportClient = MakeShareable(new FCurveEditorViewportClient(CurveEditorPtr, SharedThis(this)));

	Viewport = FSceneViewport::Create(ViewportClient, ViewportWidget);

	PrevViewportHeight = 0;
	AdjustScrollBar();
}

void SCurveEditorViewport::RefreshViewport()
{
	Viewport->Invalidate();
	Viewport->InvalidateDisplay();
}

void SCurveEditorViewport::SetVerticalScrollBarPosition(float Position)
{
	float Ratio = ViewportClient->GetViewportVerticalScrollBarRatio();
	float MaxOffset = (Ratio < 1.0f)? 1.0f - Ratio: 0.0f;
	OnViewportVerticalScrollBarScrolled(MaxOffset * Position);
}

void SCurveEditorViewport::AdjustScrollBar()
{
	// Pretend the scrollbar was scrolled by no amount; this will update the scrollbar ratio
	OnViewportVerticalScrollBarScrolled(0.0f);
}

TSharedPtr<FSceneViewport> SCurveEditorViewport::GetViewport() const
{
	return Viewport;
}

TSharedPtr<FCurveEditorViewportClient> SCurveEditorViewport::GetViewportClient() const
{
	return ViewportClient;
}

TSharedPtr<SViewport> SCurveEditorViewport::GetViewportWidget() const
{
	return ViewportWidget;
}

TSharedPtr<SScrollBar> SCurveEditorViewport::GetVerticalScrollBar() const
{
	return ViewportVerticalScrollBar;
}

void SCurveEditorViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Check to see if we need to update the scrollbars due to a size change
	const int32 CurrentHeight = AllottedGeometry.GetLocalSize().Y;
	if (CurrentHeight != PrevViewportHeight)
	{
		PrevViewportHeight = CurrentHeight;
		AdjustScrollBar();
	}
}

EVisibility SCurveEditorViewport::GetViewportVerticalScrollBarVisibility() const
{
	return (ViewportClient->GetViewportVerticalScrollBarRatio() < 1.0f)? EVisibility::Visible: EVisibility::Collapsed;
}

void SCurveEditorViewport::OnViewportVerticalScrollBarScrolled(float InScrollOffsetFraction)
{
	float Ratio = ViewportClient->GetViewportVerticalScrollBarRatio();
	float MaxOffset = (Ratio < 1.0f)? 1.0f - Ratio: 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);
	ViewportVerticalScrollBar->SetState(InScrollOffsetFraction, Ratio);
	RefreshViewport();
}

