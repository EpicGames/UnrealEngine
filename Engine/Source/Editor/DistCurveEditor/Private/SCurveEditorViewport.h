// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDistCurveEditor.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FCurveEditorViewportClient;
class FSceneViewport;
class SDistributionCurveEditor;
class SScrollBar;
class SViewport;
struct FGeometry;

/*-----------------------------------------------------------------------------
   SCurveEditorViewport
-----------------------------------------------------------------------------*/

class SCurveEditorViewport : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveEditorViewport)
		: _CurveEdOptions(IDistributionCurveEditor::FCurveEdOptions())
		{}

		SLATE_ARGUMENT(TWeakPtr<SDistributionCurveEditor>, CurveEditor)
		SLATE_ARGUMENT(IDistributionCurveEditor::FCurveEdOptions, CurveEdOptions)
	SLATE_END_ARGS()

	/** Destructor */
	~SCurveEditorViewport();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);
	
	/** Refreshes the viewport */
	void RefreshViewport();

	/** Returns true if the viewport is visible */
	bool IsVisible() const;

	/** Scrolls the scrollbar... expected range is from 0.0 to 1.0 */
	void SetVerticalScrollBarPosition(float Position);

	/**
	 * Updates the scroll bar for the current state of the window's size and content layout.  This should be called
	 * when either the window size changes or the vertical size of the content contained in the window changes.
	 */
	void AdjustScrollBar();

	/** Accessors */
	TSharedPtr<FSceneViewport> GetViewport() const;
	TSharedPtr<FCurveEditorViewportClient> GetViewportClient() const;
	TSharedPtr<SViewport> GetViewportWidget() const;
	TSharedPtr<SScrollBar> GetVerticalScrollBar() const;

protected:
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Returns the visibility of the viewport scrollbars */
	EVisibility GetViewportVerticalScrollBarVisibility() const;

	/** Called when the viewport scrollbars are scrolled */
	void OnViewportVerticalScrollBarScrolled(float InScrollOffsetFraction);

private:
	/** Pointer back to the Texture editor tool that owns us */
	TWeakPtr<SDistributionCurveEditor> CurveEditorPtr;
	
	/** Level viewport client */
	TSharedPtr<FCurveEditorViewportClient> ViewportClient;

	/** Slate viewport for rendering and I/O */
	TSharedPtr<FSceneViewport> Viewport;

	/** Viewport widget*/
	TSharedPtr<SViewport> ViewportWidget;

	/** Vertical scrollbar */
	TSharedPtr<SScrollBar> ViewportVerticalScrollBar;

	/** Height of the viewport on the last call to Tick */
	int32 PrevViewportHeight;
};
