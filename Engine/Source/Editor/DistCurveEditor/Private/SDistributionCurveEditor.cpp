// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDistributionCurveEditor.h"

#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CurveEditorActions.h"
#include "CurveEditorSharedData.h"
#include "CurveEditorViewportClient.h"
#include "Delegates/Delegate.h"
#include "Engine/Engine.h"
#include "Engine/InterpCurveEdSetup.h"
#include "Framework/Application/IMenu.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformMisc.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Math/Color.h"
#include "Math/CurveEdInterface.h"
#include "Math/InterpCurvePoint.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "SCurveEditorViewport.h"
#include "Slate/SceneViewport.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Function.h"
#include "Types/SlateStructs.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "CurveEditor"

DEFINE_LOG_CATEGORY(LogCurveEd);

SDistributionCurveEditor::SDistributionCurveEditor()
	: UICommandList(new FUICommandList())
	, FitMargin(0.1f)
{
}

SDistributionCurveEditor::~SDistributionCurveEditor()
{
}

void SDistributionCurveEditor::Construct(const FArguments& InArgs)
{
	SharedData = MakeShareable(new FCurveEditorSharedData(InArgs._EdSetup));
	SharedData->NotifyObject = InArgs._NotifyObject;

	// Register our commands. This will only register them if not previously registered
	FDistCurveEditorCommands::Register();

	for (int32 TabIdx = 0; TabIdx < SharedData->EdSetup->Tabs.Num(); TabIdx++)
	{
		FCurveEdTab* Tab = &SharedData->EdSetup->Tabs[TabIdx];
		TabNames.Add(MakeShareable(new FString(Tab->TabName)));
	}

	BindCommands();

	CreateLayout(InArgs._CurveEdOptions);
}

void SDistributionCurveEditor::RefreshViewport()
{
	Viewport->GetViewport()->Invalidate();
	Viewport->GetViewport()->InvalidateDisplay();
}

void SDistributionCurveEditor::CurveChanged()
{
	SharedData->SelectedKeys.Empty();

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::SetCurveVisible(const UObject* InCurve, bool bShow)
{
	for (int32 TabIndex = 0; TabIndex < SharedData->EdSetup->Tabs.Num(); TabIndex++)
	{
		FCurveEdTab* Tab = &(SharedData->EdSetup->Tabs[TabIndex]);
		for (int32 CurveIndex = 0; CurveIndex < Tab->Curves.Num(); CurveIndex++)
		{
			FCurveEdEntry* Entry = &(Tab->Curves[CurveIndex]);
			if (Entry->CurveObject == InCurve)
			{
				CURVEEDENTRY_SET_HIDECURVE(Entry->bHideCurve, !bShow);
				break;
			}
		}
	}
}

void SDistributionCurveEditor::ClearAllVisibleCurves()
{
	for (int32 TabIndex = 0; TabIndex < SharedData->EdSetup->Tabs.Num(); TabIndex++)
	{
		FCurveEdTab* Tab = &(SharedData->EdSetup->Tabs[TabIndex]);
		for (int32 CurveIndex = 0; CurveIndex < Tab->Curves.Num(); CurveIndex++)
		{
			FCurveEdEntry* Entry = &(Tab->Curves[CurveIndex]);
			CURVEEDENTRY_SET_HIDECURVE(Entry->bHideCurve, true);
		}
	}
}

void SDistributionCurveEditor::SetCurveSelected(const UObject* InCurve, bool bSelected)
{
	for (int32 TabIndex = 0; TabIndex < SharedData->EdSetup->Tabs.Num(); TabIndex++)
	{
		FCurveEdTab* Tab = &(SharedData->EdSetup->Tabs[TabIndex]);
		for (int32 CurveIndex = 0; CurveIndex < Tab->Curves.Num(); CurveIndex++)
		{
			FCurveEdEntry* Entry = &(Tab->Curves[CurveIndex]);
			if (Entry->CurveObject == InCurve)
			{
				CURVEEDENTRY_SET_SELECTED(Entry->bHideCurve, bSelected);
				break;
			}
		}
	}
}

void SDistributionCurveEditor::ClearAllSelectedCurves()
{
	for (int32 TabIndex = 0; TabIndex < SharedData->EdSetup->Tabs.Num(); TabIndex++)
	{
		FCurveEdTab* Tab = &(SharedData->EdSetup->Tabs[TabIndex]);
		for (int32 CurveIndex = 0; CurveIndex < Tab->Curves.Num(); CurveIndex++)
		{
			FCurveEdEntry* Entry = &(Tab->Curves[CurveIndex]);
			CURVEEDENTRY_SET_SELECTED(Entry->bHideCurve, false);
		}
	}
}

void SDistributionCurveEditor::ScrollToFirstSelected()
{
	int32 CurveCount = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num();

	if ((int32)(SharedData->LabelEntryHeight * CurveCount) < SharedData->LabelContentBoxHeight)
	{
		// All are inside the current box...
		return;
	}

	int32 SelectedIndex = -1;
	for (int32 CurveIndex = 0; CurveIndex < CurveCount; CurveIndex++)
	{
		FCurveEdEntry* Entry = &(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[CurveIndex]);
		if (CURVEEDENTRY_SELECTED(Entry->bHideCurve))
		{
			SelectedIndex = CurveIndex;
			break;
		}
	}

	if ((SelectedIndex >= 0) && (SelectedIndex < CurveCount))
	{
		Viewport->SetVerticalScrollBarPosition((float)SelectedIndex / (float)(CurveCount - 1));
	}
}

void SDistributionCurveEditor::SetActiveTabToFirstSelected()
{
	if(SharedData->EdSetup->Tabs.Num() == 1)
	{
		//There is only one tab (the default); no need to change the active tab.
		return;
	}

	//Find the Tab index for the first selected curve. We default to the current tab if no curves are selected.
	int32 TabIdx = SharedData->EdSetup->ActiveTab;
	for (int32 TabIndex = 0; TabIndex < SharedData->EdSetup->Tabs.Num(); TabIndex++)
	{
		FCurveEdTab* Tab = &(SharedData->EdSetup->Tabs[TabIndex]);
		for (int32 CurveIndex = 0; CurveIndex < Tab->Curves.Num(); CurveIndex++)
		{
			FCurveEdEntry* Entry = &(Tab->Curves[CurveIndex]);
			if (CURVEEDENTRY_SELECTED(Entry->bHideCurve))
			{
				TabIdx = TabIndex;
			}
		}
	}

	//Set the active tab and update the tab ComboBox to reflect this change.
	SharedData->EdSetup->ActiveTab = TabIdx;
}

void SDistributionCurveEditor::SetPositionMarker(bool bEnabled, float InPosition, const FColor& InMarkerColor)
{
	SharedData->bShowPositionMarker = bEnabled;
	SharedData->MarkerPosition = InPosition;
	SharedData->MarkerColor = InMarkerColor;

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::SetEndMarker(bool bEnabled, float InEndPosition)
{
	SharedData->bShowEndMarker = bEnabled;
	SharedData->EndMarkerPosition = InEndPosition;

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::SetRegionMarker(bool bEnabled, float InRegionStart, float InRegionEnd, const FColor& InRegionFillColor)
{
	SharedData->bShowRegionMarker = bEnabled;
	SharedData->RegionStart = InRegionStart;
	SharedData->RegionEnd = InRegionEnd;
	SharedData->RegionFillColor = InRegionFillColor;

	Viewport->RefreshViewport();
}

TSharedPtr<FCurveEditorSharedData> SDistributionCurveEditor::GetSharedData()
{
	return SharedData;
}

UInterpCurveEdSetup* SDistributionCurveEditor::GetEdSetup()
{
	return SharedData->EdSetup;
}

float SDistributionCurveEditor::GetStartIn()
{
	return SharedData->StartIn;
}

float SDistributionCurveEditor::GetEndIn()
{
	return SharedData->EndIn;
}

void SDistributionCurveEditor::SetViewInterval(float StartIn, float EndIn)
{
	SharedData->SetCurveView(StartIn, EndIn, SharedData->StartOut, SharedData->EndOut);

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::SetInSnap(bool bEnabled, float SnapAmount, bool bInSnapToFrames)
{
	if (Viewport.IsValid() && Viewport->GetViewportClient().IsValid())
	{
		Viewport->GetViewportClient()->SetInSnap(bEnabled, SnapAmount, bInSnapToFrames);
	}
}

void SDistributionCurveEditor::CreateLayout(FCurveEdOptions CurveEdOptions)
{
	Toolbar = BuildToolBar();

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 8.0f)
		[
			Toolbar.ToSharedRef()
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(Viewport, SCurveEditorViewport)
			.CurveEditor(SharedThis(this))
			.CurveEdOptions(CurveEdOptions)
		]
	];
}

EVisibility SDistributionCurveEditor::GetLargeIconVisibility() const
{
	return FMultiBoxSettings::UseSmallToolBarIcons.Get() ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SHorizontalBox> SDistributionCurveEditor::BuildToolBar()
{
	SelectedTab = TabNames[0];

	FToolBarBuilder ToolbarBuilder( UICommandList, FMultiBoxCustomization::None );
	ToolbarBuilder.BeginSection("CurveEditorFit");
	{
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().FitHorizontally);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().FitVertically);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().Fit);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("CurveEditorMode");
	{
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().PanMode);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().ZoomMode);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("CurveEditorTangentTypes");
	{
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().CurveAuto);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().CurveAutoClamped);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().CurveUser);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().CurveBreak);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().Linear);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().Constant);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("CurveEditorTangentOptions");
	{
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().FlattenTangents);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().StraightenTangents);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().ShowAllTangents);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("CurveEditorTabs");
	{
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().CreateTab);
		ToolbarBuilder.AddToolBarButton(FDistCurveEditorCommands::Get().DeleteTab);
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.WidthOverride(175)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(4)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CurrentTab", "Current Tab: "))
					.Visibility( this, &SDistributionCurveEditor::GetLargeIconVisibility )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4,0)
				[
					SAssignNew(TabNamesComboBox, STextComboBox)
					.OptionsSource(&TabNames)
					.OnSelectionChanged(this, &SDistributionCurveEditor::TabSelectionChanged)
					.InitiallySelectedItem(SelectedTab)
				]
			]
		);
	}
	ToolbarBuilder.EndSection();

	return
	SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.Padding(4,0)
	[
		SNew(SBorder)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		[
			ToolbarBuilder.MakeWidget()
		]
	];
}

void SDistributionCurveEditor::BindCommands()
{
	const FDistCurveEditorCommands& Commands = FDistCurveEditorCommands::Get();

	UICommandList->MapAction(
		Commands.RemoveCurve,
		FExecuteAction::CreateSP(SharedThis(this), &SDistributionCurveEditor::OnRemoveCurve));

	UICommandList->MapAction(
		Commands.RemoveAllCurves,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnRemoveAllCurves));

	UICommandList->MapAction(
		Commands.SetTime,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetTime));

	UICommandList->MapAction(
		Commands.SetValue,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetValue));

	UICommandList->MapAction(
		Commands.SetColor,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetColor));

	UICommandList->MapAction(
		Commands.DeleteKeys,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnDeleteKeys));

	UICommandList->MapAction(
		Commands.ScaleTimes,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnScaleTimes, ECurveScaleScope::All));

	UICommandList->MapAction(
		Commands.ScaleValues,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnScaleValues, ECurveScaleScope::All));

	UICommandList->MapAction(
		Commands.ScaleSingleCurveTimes,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnScaleTimes, ECurveScaleScope::Current));

	UICommandList->MapAction(
		Commands.ScaleSingleCurveValues,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnScaleValues, ECurveScaleScope::Current));

	UICommandList->MapAction(
		Commands.ScaleSingleSubCurveValues,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnScaleValues, ECurveScaleScope::CurrentSub));

	UICommandList->MapAction(
		Commands.FitHorizontally,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnFitHorizontally));

	UICommandList->MapAction(
		Commands.FitVertically,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnFitVertically));

	UICommandList->MapAction(
		Commands.Fit,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnFit));

	UICommandList->MapAction(
		Commands.PanMode,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetMode, (int32)FCurveEditorSharedData::CEM_Pan),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsModeChecked, (int32)FCurveEditorSharedData::CEM_Pan));

	UICommandList->MapAction(
		Commands.ZoomMode,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetMode, (int32)FCurveEditorSharedData::CEM_Zoom),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsModeChecked, (int32)FCurveEditorSharedData::CEM_Zoom));

	UICommandList->MapAction(
		Commands.CurveAuto,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetTangentType, (int32)CIM_CurveAuto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsTangentTypeChecked, (int32)CIM_CurveAuto));

	UICommandList->MapAction(
		Commands.CurveAutoClamped,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetTangentType, (int32)CIM_CurveAutoClamped),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsTangentTypeChecked, (int32)CIM_CurveAutoClamped));

	UICommandList->MapAction(
		Commands.CurveUser,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetTangentType, (int32)CIM_CurveUser),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsTangentTypeChecked, (int32)CIM_CurveUser));

	UICommandList->MapAction(
		Commands.CurveBreak,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetTangentType, (int32)CIM_CurveBreak),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsTangentTypeChecked, (int32)CIM_CurveBreak));

	UICommandList->MapAction(
		Commands.Linear,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetTangentType, (int32)CIM_Linear),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsTangentTypeChecked, (int32)CIM_Linear));

	UICommandList->MapAction(
		Commands.Constant,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnSetTangentType, (int32)CIM_Constant),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsTangentTypeChecked, (int32)CIM_Constant));

	UICommandList->MapAction(
		Commands.FlattenTangents,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnFlattenTangents));

	UICommandList->MapAction(
		Commands.StraightenTangents,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnStraightenTangents));

	UICommandList->MapAction(
		Commands.ShowAllTangents,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnShowAllTangents),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDistributionCurveEditor::IsShowAllTangentsChecked));

	UICommandList->MapAction(
		Commands.CreateTab,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnCreateTab));

	UICommandList->MapAction(
		Commands.DeleteTab,
		FExecuteAction::CreateSP(this, &SDistributionCurveEditor::OnDeleteTab));
}

void SDistributionCurveEditor::OnRemoveCurve()
{
	if(SharedData->RightClickCurveIndex < 0 || SharedData->RightClickCurveIndex >= SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num())
	{
		return;
	}

	SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.RemoveAt(SharedData->RightClickCurveIndex);

	SharedData->SelectedKeys.Empty();

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::OnRemoveAllCurves()
{
	bool bShouldPromptOnCurveRemoveAll;
	GConfig->GetBool(TEXT("CurveEditor"), TEXT("bShouldPromptOnCurveRemoveAll"), bShouldPromptOnCurveRemoveAll, GEditorIni);

	if (!bShouldPromptOnCurveRemoveAll || EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RemoveAllCurvesPrompt", "Are you sure you want to 'Remove All Curves'?")))
	{
		for (int32 TabIndex = 0; TabIndex < SharedData->EdSetup->Tabs.Num(); TabIndex++)
		{
			SharedData->EdSetup->Tabs[TabIndex].Curves.Empty();
		}

		SharedData->SelectedKeys.Empty();
		Viewport->RefreshViewport();
	}
}

void SDistributionCurveEditor::OnSetTime()
{
	FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[0];
	FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
	FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

	FString DefaultText = FString::Printf(TEXT("%.2f"), EdInterface->GetKeyIn(SelKey.KeyIndex));

	TSharedRef<STextEntryPopup> TextEntry = 
		SNew(STextEntryPopup)
		.Label(LOCTEXT("SetTime", "Time: "))
		.DefaultText(FText::FromString( DefaultText ) )
		.OnTextCommitted(this, &SDistributionCurveEditor::KeyTimeCommitted)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false);

	EntryMenu = FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SDistributionCurveEditor::OnSetValue()
{
	FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[0];
	FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
	FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

	FString DefaultText = FString::Printf(TEXT("%.2f"), EdInterface->GetKeyOut(SelKey.SubIndex, SelKey.KeyIndex));

	TSharedRef<STextEntryPopup> TextEntry = 
		SNew(STextEntryPopup)
		.Label(LOCTEXT("SetValue", "Value: "))
		.DefaultText(FText::FromString(DefaultText))
		.OnTextCommitted(this, &SDistributionCurveEditor::KeyValueCommitted)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false);

	EntryMenu = FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SDistributionCurveEditor::OnSetColor()
{
	// Only works on single key...
	if(SharedData->SelectedKeys.Num() != 1)
		return;

	// Find the EdInterface for this curve.
	FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[0];
	FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
	if(!Entry.bColorCurve)
		return;

	// We only do this special case if curve has 3 sub-curves.
	FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
	if(EdInterface->GetNumSubCurves() != 3)
		return;

	if (SharedData->NotifyObject)
	{
		// Make a list of all curves we are going to remove keys from.
		TArray<UObject*> CurvesAboutToChange;
		if(Entry.CurveObject)
		{
			CurvesAboutToChange.AddUnique(Entry.CurveObject);
			// Notify a containing tool that keys are about to be removed
			SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);
		}
	}

	// Get current value of curve as a colour.
	FColor InputColor;
	if (Entry.bFloatingPointColorCurve)
	{
		float Value;

		Value	= EdInterface->GetKeyOut(0, SelKey.KeyIndex) * 255.f;
		InputColor.R = FMath::RoundToInt(Value);
		Value	= EdInterface->GetKeyOut(1, SelKey.KeyIndex) * 255.f;
		InputColor.G = FMath::RoundToInt(Value);
		Value	= EdInterface->GetKeyOut(2, SelKey.KeyIndex) * 255.f;
		InputColor.B = FMath::RoundToInt(Value);
	}
	else
	{
		InputColor.R = FMath::RoundToInt(FMath::Clamp<float>(EdInterface->GetKeyOut(0, SelKey.KeyIndex), 0.f, 255.f));
		InputColor.G = FMath::RoundToInt(FMath::Clamp<float>(EdInterface->GetKeyOut(1, SelKey.KeyIndex), 0.f, 255.f));
		InputColor.B = FMath::RoundToInt(FMath::Clamp<float>(EdInterface->GetKeyOut(2, SelKey.KeyIndex), 0.f, 255.f));
	}

	//since the data isn't stored in standard colors, a temp color is used
	FLinearColor TempColor = InputColor;
	FColorPickerArgs PickerArgs = FColorPickerArgs(InputColor, FOnLinearColorValueChanged::CreateLambda([&TempColor](FLinearColor NewValue){ TempColor = NewValue.ToFColorSRGB(); }));
	PickerArgs.bIsModal = true;
	PickerArgs.bClampValue = true;
	PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );

	if (OpenColorPicker(PickerArgs))
	{
		float Value;
		if (Entry.bFloatingPointColorCurve)
		{
			Value	= (float)TempColor.R / 255.f;
			if (Entry.bClamp)
			{
				Value = FMath::Clamp<float>(Value, Entry.ClampLow, Entry.ClampHigh);
			}
			EdInterface->SetKeyOut(0, SelKey.KeyIndex, Value);
			Value	= (float)TempColor.G / 255.f;
			if (Entry.bClamp)
			{
				Value = FMath::Clamp<float>(Value, Entry.ClampLow, Entry.ClampHigh);
			}
			EdInterface->SetKeyOut(1, SelKey.KeyIndex, Value);
			Value	= (float)TempColor.B / 255.f;
			if (Entry.bClamp)
			{
				Value = FMath::Clamp<float>(Value, Entry.ClampLow, Entry.ClampHigh);
			}
			EdInterface->SetKeyOut(2, SelKey.KeyIndex, Value);
		}
		else
		{
			Value = (float)(TempColor.R);
			if (Entry.bClamp)
			{
				Value = FMath::Clamp<float>(Value, Entry.ClampLow, Entry.ClampHigh);
			}
			EdInterface->SetKeyOut(0, SelKey.KeyIndex, Value);
			Value = (float)(TempColor.G);
			if (Entry.bClamp)
			{
				Value = FMath::Clamp<float>(Value, Entry.ClampLow, Entry.ClampHigh);
			}
			EdInterface->SetKeyOut(1, SelKey.KeyIndex, Value);
			Value = (float)(TempColor.B);
			if (Entry.bClamp)
			{
				Value = FMath::Clamp<float>(Value, Entry.ClampLow, Entry.ClampHigh);
			}
			EdInterface->SetKeyOut(2, SelKey.KeyIndex, Value);
		}
	}

	if (SharedData->NotifyObject)
	{
		SharedData->NotifyObject->PostEditCurve();
	}

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::OnDeleteKeys()
{
	// Make a list of all curves we are going to remove keys from.
	TArray<UObject*> CurvesAboutToChange;
	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[i];
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];

		if(Entry.CurveObject)
		{
			CurvesAboutToChange.AddUnique(Entry.CurveObject);
		}
	}

	// Notify a containing tool that keys are about to be removed
	if(SharedData->NotifyObject)
	{
		SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);
	}

	// Iterate over selected keys and actually remove them.
	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[i];

		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

		EdInterface->DeleteKey(SelKey.KeyIndex);

		// Do any updating on the rest of the selection.
		int32 j=i+1;
		while(j<SharedData->SelectedKeys.Num())
		{
			// If key is on same curve..
			if(SharedData->SelectedKeys[j].CurveIndex == SelKey.CurveIndex)
			{
				// If key is same curve and key, but different sub, remove it.
				if(SharedData->SelectedKeys[j].KeyIndex == SelKey.KeyIndex)
				{
					SharedData->SelectedKeys.RemoveAt(j);
				}
				// If its on same curve but higher key index, decrement it
				else if(SharedData->SelectedKeys[j].KeyIndex > SelKey.KeyIndex)
				{
					SharedData->SelectedKeys[j].KeyIndex--;
					j++;
				}
				// Otherwise, do nothing.
				else
				{
					j++;
				}
			}
			// Otherwise, do nothing.
			else
			{
				j++;
			}
		}
	}

	if(SharedData->NotifyObject)
	{
		SharedData->NotifyObject->PostEditCurve();
	}

	// Finally deselect everything.
	SharedData->SelectedKeys.Empty();

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::OnScaleTimes(ECurveScaleScope::Type Scope)
{
	FString DefaultText = FString::Printf(TEXT("%.2f"), 1.0f);

	FText Label;
	if(Scope == ECurveScaleScope::All)
	{
		Label = LOCTEXT("ScaleTimeAll", "Time Scale (All): ");
	}
	else if(Scope == ECurveScaleScope::Current || Scope == ECurveScaleScope::CurrentSub)
	{
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SharedData->RightClickCurveIndex];
		Label = FText::Format(LOCTEXT("ScaleTime", "Time Scale ({0}): "), FText::FromString(Entry.CurveName));
	}

	TSharedRef<STextEntryPopup> TextEntry = 
		SNew(STextEntryPopup)
		.Label(Label)
		.DefaultText(FText::FromString( DefaultText ))
		.OnTextCommitted(this, &SDistributionCurveEditor::ScaleTimeCommitted, Scope)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false);

	EntryMenu = FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SDistributionCurveEditor::OnScaleValues(ECurveScaleScope::Type Scope)
{
	FString DefaultText = FString::Printf(TEXT("%.2f"), 1.0f);

	FText Label;
	if(Scope == ECurveScaleScope::All)
	{
		Label = LOCTEXT("ScaleValueAll", "Scale Values (All): ");
	}
	else if(Scope == ECurveScaleScope::Current)
	{
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SharedData->RightClickCurveIndex];
		Label = FText::Format(LOCTEXT("ScaleValue", "Scale Values ({0}): "), FText::FromString(Entry.CurveName));
	}
	else if( Scope == ECurveScaleScope::CurrentSub)
	{
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SharedData->RightClickCurveIndex];
		Label = FText::Format(LOCTEXT("ScaleSubValue", "Scale Sub-Value ({0}:{1}): "), FText::FromString(Entry.CurveName), FText::AsNumber(SharedData->RightClickCurveSubIndex));
	}

	TSharedRef<STextEntryPopup> TextEntry = 
		SNew(STextEntryPopup)
		.Label(Label)
		.DefaultText(FText::FromString( DefaultText ))
		.OnTextCommitted(this, &SDistributionCurveEditor::ScaleValueCommitted, Scope)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false);

	EntryMenu = FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SDistributionCurveEditor::OnFitHorizontally()
{
	FitViewHorizontally();
}

void SDistributionCurveEditor::OnFitVertically()
{
	FitViewVertically();
}

void SDistributionCurveEditor::OnFit()
{
	FitViewHorizontally();
	FitViewVertically();
}

void SDistributionCurveEditor::OnFitToSelected()
{
	float MinOut = BIG_NUMBER;
	float MaxOut = -BIG_NUMBER;
	float MinIn = BIG_NUMBER;
	float MaxIn = -BIG_NUMBER;

	for(int32 i = 0; i < SharedData->SelectedKeys.Num(); ++i)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[ i ];

		FCurveEdEntry& CurveEntry = SharedData->EdSetup->Tabs[ SharedData->EdSetup->ActiveTab ].Curves[ SelKey.CurveIndex ];
		FCurveEdInterface* CurveInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(CurveEntry);

		if(!CURVEEDENTRY_HIDESUBCURVE(CurveEntry.bHideCurve, SelKey.SubIndex))
		{
			const float KeyIn = CurveInterface->GetKeyIn(SelKey.KeyIndex);
			const float KeyOut = CurveInterface->GetKeyOut(SelKey.SubIndex, SelKey.KeyIndex);

			// Update overall min and max
			MinOut = FMath::Min<float>(KeyOut, MinOut);
			MaxOut = FMath::Max<float>(KeyOut, MaxOut);
			MinIn = FMath::Min<float>(KeyIn, MinIn);
			MaxIn = FMath::Max<float>(KeyIn, MaxIn);
		}
	}

	float SizeOut = MaxOut - MinOut;
	float SizeIn = MaxIn - MinIn;

	// Clamp the minimum size
	if(SizeOut < SharedData->MinViewRange)
	{
		MinOut -= SharedData->MinViewRange * 0.5f;
		MaxOut += SharedData->MinViewRange * 0.5f;
		SizeOut = MaxOut - MinOut;
	}
	if(SizeIn < SharedData->MinViewRange)
	{
		MinIn -= SharedData->MinViewRange * 0.5f;
		MaxIn += SharedData->MinViewRange * 0.5f;
		SizeIn = MaxIn - MinIn;
	}

	SharedData->SetCurveView(
		MinIn - FitMargin * SizeIn,
		MaxIn + FitMargin * SizeIn,
		MinOut - FitMargin * SizeOut,
		MaxOut + FitMargin * SizeOut);

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::OnSetMode(int32 NewMode)
{
	SharedData->EdMode = (FCurveEditorSharedData::ECurveEdMode)NewMode;
}

bool SDistributionCurveEditor::IsModeChecked(int32 Mode) const
{
	return (FCurveEditorSharedData::ECurveEdMode)Mode == SharedData->EdMode;
}

void SDistributionCurveEditor::OnSetTangentType(int32 NewType)
{
	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[i];
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);		

		EdInterface->SetKeyInterpMode(SelKey.KeyIndex, (EInterpCurveMode)NewType);
	}

	Viewport->RefreshViewport();
}

bool SDistributionCurveEditor::IsTangentTypeChecked(int32 Type) const
{
	if (SharedData->SelectedKeys.Num() == 0)
	{
		return false;
	}

	EInterpCurveMode Mode = (EInterpCurveMode)Type;
	for(int32 i=0; i<SharedData->SelectedKeys.Num(); i++)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[i];
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

		if (Mode != EdInterface->GetKeyInterpMode(SelKey.KeyIndex))
		{
			return false;
		}
	}
	return true;
}

void SDistributionCurveEditor::OnFlattenTangents()
{
	bool bDoStraighten = false;
	ModifyTangents(bDoStraighten);
}

void SDistributionCurveEditor::OnStraightenTangents()
{
	bool bDoStraighten = true;
	ModifyTangents(bDoStraighten);
}

void SDistributionCurveEditor::OnShowAllTangents()
{
	SharedData->bShowAllCurveTangents = !SharedData->bShowAllCurveTangents;

	Viewport->RefreshViewport();
}

bool SDistributionCurveEditor::IsShowAllTangentsChecked() const
{
	return SharedData->bShowAllCurveTangents;
}

void SDistributionCurveEditor::OnCreateTab()
{
	TSharedRef<STextEntryPopup> TextEntry = 
		SNew(STextEntryPopup)
		.Label(LOCTEXT("SetTabName", "Tab Name: "))
		.OnTextCommitted(this, &SDistributionCurveEditor::TabNameCommitted)
		.ClearKeyboardFocusOnCommit(false);

	EntryMenu = FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SDistributionCurveEditor::OnDeleteTab()
{
	if (TabNamesComboBox->GetSelectedItem().IsValid())
	{
		if (TabNamesComboBox->GetSelectedItem() == TabNames[0])
		{
			FSlateNotificationManager::Get().AddNotification( FNotificationInfo( LOCTEXT("DefaultTabCannotBeDeleted", "Default tab can not be deleted!") ) );
			return;
		}

		// Remove the tab...
		FString Name = *TabNamesComboBox->GetSelectedItem();
		SharedData->EdSetup->RemoveTab(Name);
		TabNames.Remove(TabNamesComboBox->GetSelectedItem());

		// Force a reset of the combo contents
		SelectedTab = TabNames[0];
		TabNamesComboBox->RefreshOptions();

		SetTabSelection(SelectedTab, true);
	}
}

void SDistributionCurveEditor::OpenLabelMenu()
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		SharedThis( this ),
		FWidgetPath(),
		BuildMenuWidgetLabel(),
		MouseCursorLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
}

void SDistributionCurveEditor::OpenKeyMenu()
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		SharedThis( this ),
		FWidgetPath(),
		BuildMenuWidgetKey(),
		MouseCursorLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
}

void SDistributionCurveEditor::OpenGeneralMenu()
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		SharedThis( this ),
		FWidgetPath(),
		BuildMenuWidgetGeneral(),
		MouseCursorLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
}

void SDistributionCurveEditor::OpenCurveMenu()
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		SharedThis( this ),
		FWidgetPath(),
		BuildMenuWidgetCurve(),
		MouseCursorLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
}

void SDistributionCurveEditor::CloseEntryPopup()
{
	if (EntryMenu.IsValid())
	{
		EntryMenu.Pin()->Dismiss();
	}
}

TSharedRef<SWidget> SDistributionCurveEditor::BuildMenuWidgetLabel()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);
	{
		MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().RemoveCurve);
		MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().RemoveAllCurves);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SDistributionCurveEditor::BuildMenuWidgetKey()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);
	{
		MenuBuilder.BeginSection("DistributionCurveWidgetKey");
		{
			if(SharedData->SelectedKeys.Num() == 1)
			{
				MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().SetTime);
				MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().SetValue);

				FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[0];
				FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
				FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

				if(Entry.bColorCurve && EdInterface->GetNumSubCurves() == 3)
				{
					MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().SetColor);
				}
			}
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("DistributionCurveWidgetKey2");
		{
			MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().DeleteKeys);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SDistributionCurveEditor::BuildMenuWidgetGeneral()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);

	MenuBuilder.BeginSection("AllCurvesSection", LOCTEXT("AllCurvesMenuHeader", "All Curves"));
	{
		MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().ScaleTimes);
		MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().ScaleValues);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SDistributionCurveEditor::BuildMenuWidgetCurve()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);
	{
		MenuBuilder.BeginSection("AllCurvesSection", LOCTEXT("AllCurvesMenuHeader", "All Curves"));
		{
			MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().ScaleTimes);
			MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().ScaleValues);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("CurrentCurveSection", LOCTEXT("CurrentCurveMenuHeader", "Current Curve"));
		{
			MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().ScaleSingleCurveTimes);
			MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().ScaleSingleCurveValues);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("SubCurveSection", LOCTEXT("SubCurveMenuHeader", "Sub-Curve"));
		{
			MenuBuilder.AddMenuEntry(FDistCurveEditorCommands::Get().ScaleSingleSubCurveValues);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SDistributionCurveEditor::TabSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	SetTabSelection(NewSelection, false);
}

void SDistributionCurveEditor::SetTabSelection(TSharedPtr<FString> NewSelection, bool bUpdateWidget)
{
	for (int32 TabIdx = 0; TabIdx < SharedData->EdSetup->Tabs.Num(); TabIdx++)
	{
		FCurveEdTab* Tab = &SharedData->EdSetup->Tabs[TabIdx];
		if (Tab->TabName == *NewSelection)
		{
			SharedData->EdSetup->ActiveTab = TabIdx;
			SharedData->SelectedKeys.Empty();
			Viewport->RefreshViewport();

			if (bUpdateWidget)
			{
				TabNamesComboBox->SetSelectedItem(NewSelection);
			}

			return;
		}
	}

	// The combobox and the tabs are out of sync if this gets hit
	check(false);
}

void SDistributionCurveEditor::KeyTimeCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[0];
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
		
		if (SharedData->NotifyObject)
		{
			// Make a list of all curves we are going to remove keys from.
			TArray<UObject*> CurvesAboutToChange;
			if(Entry.CurveObject)
			{
				CurvesAboutToChange.AddUnique(Entry.CurveObject);
				// Notify a containing tool that keys are about to be removed
				SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);
			}
		}

		// Set then set using EdInterface.
		EdInterface->SetKeyIn(SelKey.KeyIndex, atof(TCHAR_TO_ANSI( *CommentText.ToString() )));

		if (SharedData->NotifyObject)
		{
			SharedData->NotifyObject->PostEditCurve();
		}

		Viewport->RefreshViewport();
	}

	CloseEntryPopup();
}

void SDistributionCurveEditor::KeyValueCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[0];
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SelKey.CurveIndex];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

		if (SharedData->NotifyObject)
		{
			// Make a list of all curves we are going to remove keys from.
			TArray<UObject*> CurvesAboutToChange;
			if(Entry.CurveObject)
			{
				CurvesAboutToChange.AddUnique(Entry.CurveObject);
				// Notify a containing tool that keys are about to be removed
				SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);
			}
		}

		// Set then set using EdInterface.
		float NewNum = atof(TCHAR_TO_ANSI( *CommentText.ToString() ));
		if (Entry.bClamp)
		{
			NewNum = FMath::Clamp<float>(NewNum, Entry.ClampLow, Entry.ClampHigh);
		}
		EdInterface->SetKeyOut(SelKey.SubIndex, SelKey.KeyIndex, NewNum);
		
		if (SharedData->NotifyObject)
		{
			SharedData->NotifyObject->PostEditCurve();
		}

		Viewport->RefreshViewport();
	}

	CloseEntryPopup();
}

void SDistributionCurveEditor::ScaleTimeCommitted(const FText& CommentText, ETextCommit::Type CommitInfo, ECurveScaleScope::Type Scope)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		float ScaleByValue = atof(TCHAR_TO_ANSI( *CommentText.ToString() ));
		bool bNotified = NotifyPendingCurveChange(false);

		struct Local
		{
			static void ScaleCurveTime(FCurveEdEntry& Entry, float InScaleByValue)
			{
				FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
				if(EdInterface)
				{
					// For each key
					if (InScaleByValue >= 1.0f)
					{
						for (int32 KeyIndex = EdInterface->GetNumKeys() - 1; KeyIndex >= 0; KeyIndex--)
						{
							float InVal = EdInterface->GetKeyIn(KeyIndex);
							EdInterface->SetKeyIn(KeyIndex, InVal * InScaleByValue);
						}
					}
					else
					{
						for (int32 KeyIndex = 0; KeyIndex < EdInterface->GetNumKeys(); KeyIndex++)
						{
							float InVal = EdInterface->GetKeyIn(KeyIndex);
							EdInterface->SetKeyIn(KeyIndex, InVal * InScaleByValue);
						}
					}
				}
			}
		};

		// Scale the In values by the selected scalar
		if(Scope == ECurveScaleScope::All)
		{
			// Scale the In values by the selected scalar
			for (int32 CurveIdx = 0; CurveIdx < SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num(); CurveIdx++)
			{
				Local::ScaleCurveTime(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[CurveIdx], ScaleByValue);
			}
		}
		else if(Scope == ECurveScaleScope::Current || Scope == ECurveScaleScope::CurrentSub)
		{
			// we cant scale times differently for sub-curves, as they share their key times
			check(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.IsValidIndex(SharedData->RightClickCurveIndex));
			Local::ScaleCurveTime(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SharedData->RightClickCurveIndex], ScaleByValue);			
		}

		if (bNotified && SharedData->NotifyObject)
		{
			SharedData->NotifyObject->PostEditCurve();
		}

		Viewport->RefreshViewport();
	}

	CloseEntryPopup();
}


void SDistributionCurveEditor::ScaleValueCommitted(const FText& CommentText, ETextCommit::Type CommitInfo, ECurveScaleScope::Type Scope)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		float ScaleByValue = atof(TCHAR_TO_ANSI( *CommentText.ToString() ));
		bool bNotified = NotifyPendingCurveChange(false);

		struct Local
		{
			static void ScaleCurveValue(FCurveEdEntry& Entry, int32 SubCurve, float InScaleByValue)
			{
				FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);
				if(EdInterface)
				{
					if(SubCurve != INDEX_NONE)
					{
						check(SubCurve >= 0);
						check(SubCurve < EdInterface->GetNumSubCurves());

						// For each key
						for (int32 KeyIndex = 0; KeyIndex < EdInterface->GetNumKeys(); KeyIndex++)
						{
							float OutVal = EdInterface->GetKeyOut(SubCurve, KeyIndex);
							EdInterface->SetKeyOut(SubCurve, KeyIndex, OutVal * InScaleByValue);
						}
					}
					else
					{
						// For each sub-curve
						for (int32 SubIndex = 0; SubIndex < EdInterface->GetNumSubCurves(); SubIndex++)
						{
							// For each key
							for (int32 KeyIndex = 0; KeyIndex < EdInterface->GetNumKeys(); KeyIndex++)
							{
								float OutVal = EdInterface->GetKeyOut(SubIndex, KeyIndex);
								EdInterface->SetKeyOut(SubIndex, KeyIndex, OutVal * InScaleByValue);
							}
						}
					}
				}
			}
		};

		// Scale the In values by the selected scalar
		if(Scope == ECurveScaleScope::All)
		{
			for (int32 CurveIdx = 0; CurveIdx < SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num(); CurveIdx++)
			{
				Local::ScaleCurveValue(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[CurveIdx], INDEX_NONE, ScaleByValue);
			}
		}
		else if(Scope == ECurveScaleScope::Current)
		{
			check(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.IsValidIndex(SharedData->RightClickCurveIndex));
			Local::ScaleCurveValue(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SharedData->RightClickCurveIndex], INDEX_NONE, ScaleByValue);
		}
		else if(Scope == ECurveScaleScope::CurrentSub)
		{
			check(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.IsValidIndex(SharedData->RightClickCurveIndex));
			Local::ScaleCurveValue(SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[SharedData->RightClickCurveIndex], SharedData->RightClickCurveSubIndex, ScaleByValue);
		}

		if (bNotified && SharedData->NotifyObject)
		{
			SharedData->NotifyObject->PostEditCurve();
		}

		Viewport->RefreshViewport();
	}

	CloseEntryPopup();
}

void SDistributionCurveEditor::TabNameCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (CommentText.IsEmpty())
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText( LOCTEXT( "EmptyTabName", "Tab must be given a name") ) );
		}
		else
		{
			bool bFound = false;

			// Verify that the name is not already in use
			for (int32 TabIndex = 0; TabIndex < SharedData->EdSetup->Tabs.Num(); TabIndex++)
			{
				FCurveEdTab* Tab = &SharedData->EdSetup->Tabs[TabIndex];
				if (Tab->TabName == CommentText.ToString())
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("Name"), CommentText);
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format( LOCTEXT( "TabNameInUse", "Name '{Name}' already in use!" ), Arguments ));
					bFound	= true;
					break;
				}
			}

			if (!bFound)
			{
				// Add the tab, and set the active tab to it.
				SharedData->EdSetup->CreateNewTab( *CommentText.ToString() );
				SharedData->EdSetup->ActiveTab = TabNames.Num();
				TabNames.Add(MakeShareable(new FString( *CommentText.ToString() )));
				SelectedTab = TabNames[TabNames.Num() - 1];
				TabNamesComboBox->RefreshOptions();

				SetTabSelection(SelectedTab, true);
			}
		}
	}

	CloseEntryPopup();
}

bool SDistributionCurveEditor::NotifyPendingCurveChange(bool bSelectedOnly)
{
	if (SharedData->NotifyObject)
	{
		// Make a list of all curves we are going to remove keys from.
		TArray<UObject*> CurvesAboutToChange;
		for (int32 CurveIdx = 0; CurveIdx < SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves.Num(); CurveIdx++)
		{
			FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves[CurveIdx];
			if (Entry.CurveObject)
			{
				CurvesAboutToChange.AddUnique(Entry.CurveObject);
			}
		}
		// Notify a containing tool that keys are about to be removed
		SharedData->NotifyObject->PreEditCurve(CurvesAboutToChange);

		return true;
	}

	return false;
}

void SDistributionCurveEditor::FitViewHorizontally()
{
	float MinIn = BIG_NUMBER;
	float MaxIn = -BIG_NUMBER;

	IterateKeys([&](int32 KeyIndex, int32 SubCurveIndex, FCurveEdEntry& CurveEntry, FCurveEdInterface& EdInterface){

		const float KeyIn = EdInterface.GetKeyIn(KeyIndex);

		// Update overall min and max
		MinIn = FMath::Min<float>(KeyIn, MinIn);
		MaxIn = FMath::Max<float>(KeyIn, MaxIn);
		
	});

	float Size = MaxIn - MinIn;

	// Clamp the minimum size
	if(Size < SharedData->MinViewRange)
	{
		MinIn -= 0.005f;
		MaxIn += 0.005f;
		Size = MaxIn - MinIn;
	}

	SharedData->SetCurveView(MinIn - FitMargin*Size, MaxIn + FitMargin*Size, SharedData->StartOut, SharedData->EndOut);

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::IterateKeys(TFunctionRef<void(int32, int32, FCurveEdEntry&, FCurveEdInterface&)> IteratorCallback)
{
	if (SharedData->SelectedKeys.Num() == 0)
	{
		for (FCurveEdEntry& Entry : SharedData->EdSetup->Tabs[SharedData->EdSetup->ActiveTab].Curves)
		{
			if (CURVEEDENTRY_HIDECURVE(Entry.bHideCurve))
			{
				continue;
			}
			
			if (FCurveEdInterface* CurveInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry))
			{
				// Iterate over each subcurve - only looking at points which are shown
				for(int32 SubIndex = 0; SubIndex < CurveInterface->GetNumSubCurves(); SubIndex++)
				{
					if (CURVEEDENTRY_HIDESUBCURVE(Entry.bHideCurve, SubIndex))
					{
						continue;
					}

					// If we can see this curve - iterate over keys to find min and max 'out' value
					for(int32 KeyIndex = 0; KeyIndex < CurveInterface->GetNumKeys(); KeyIndex++)
					{
						IteratorCallback(KeyIndex, SubIndex, Entry, *CurveInterface);
					}
				}
			}
		}
	}
	else for (FCurveEditorSelectedKey& SelKey : SharedData->SelectedKeys)
	{
		FCurveEdEntry& CurveEntry = SharedData->EdSetup->Tabs[ SharedData->EdSetup->ActiveTab ].Curves[ SelKey.CurveIndex ];
		
		if (CURVEEDENTRY_HIDESUBCURVE(CurveEntry.bHideCurve, SelKey.SubIndex))
		{
			continue;
		}
		else if (FCurveEdInterface* CurveInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(CurveEntry))
		{
			IteratorCallback(SelKey.KeyIndex, SelKey.SubIndex, CurveEntry, *CurveInterface);
		}
	}
}

void SDistributionCurveEditor::FitViewVertically()
{
	float MinOut = BIG_NUMBER;
	float MaxOut = -BIG_NUMBER;

	IterateKeys([&](int32 KeyIndex, int32 SubCurveIndex, FCurveEdEntry& CurveEntry, FCurveEdInterface& EdInterface){

		const float KeyOut = EdInterface.GetKeyOut(SubCurveIndex, KeyIndex);

		// Update overall min and max
		MinOut = FMath::Min<float>(KeyOut, MinOut);
		MaxOut = FMath::Max<float>(KeyOut, MaxOut);

	});

	float Size = MaxOut - MinOut;

	// Clamp the minimum size
	if(Size < SharedData->MinViewRange)
	{
		MinOut -= 0.005f;
		MaxOut += 0.005f;
		Size = MaxOut - MinOut;
	}

	SharedData->SetCurveView(SharedData->StartIn, SharedData->EndIn, MinOut - FitMargin*Size, MaxOut + FitMargin*Size);

	Viewport->RefreshViewport();
}

void SDistributionCurveEditor::ModifyTangents(bool bDoStraighten)
{
	for(int32 i = 0; i < SharedData->SelectedKeys.Num(); ++i)
	{
		FCurveEditorSelectedKey& SelKey = SharedData->SelectedKeys[ i ];
		FCurveEdEntry& Entry = SharedData->EdSetup->Tabs[ SharedData->EdSetup->ActiveTab ].Curves[ SelKey.CurveIndex ];
		FCurveEdInterface* EdInterface = UInterpCurveEdSetup::GetCurveEdInterfacePointer(Entry);

		// If we're in auto-curve mode, change the interp mode to USER
		EInterpCurveMode CurInterpMode = EdInterface->GetKeyInterpMode(SelKey.KeyIndex);
		if(CurInterpMode == CIM_CurveAuto || CurInterpMode == CIM_CurveAutoClamped)
		{
			EdInterface->SetKeyInterpMode(SelKey.KeyIndex, CIM_CurveUser);
		}

		if (bDoStraighten)
		{
			// Grab the current incoming and outgoing tangent vectors
			float CurInTangent, CurOutTangent;
			EdInterface->GetTangents(SelKey.SubIndex, SelKey.KeyIndex, CurInTangent, CurOutTangent);

			// Average the tangents
			float StraightTangent = (CurInTangent + CurOutTangent) * 0.5f;

			// Straighten the tangents out!
			EdInterface->SetTangents(SelKey.SubIndex, SelKey.KeyIndex, StraightTangent, StraightTangent);
		}
		else
		{
			// Flatten the tangents along the horizontal axis by zeroing out their slope
			EdInterface->SetTangents(SelKey.SubIndex, SelKey.KeyIndex, 0.0f, 0.0f);
		}
	}

	Viewport->RefreshViewport();
}

TSharedPtr<FString> SDistributionCurveEditor::GetSelectedTab() const
{
	return SelectedTab;
}

#undef LOCTEXT_NAMESPACE
