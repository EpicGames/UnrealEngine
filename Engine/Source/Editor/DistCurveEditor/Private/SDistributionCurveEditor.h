// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/PlatformMath.h"
#include "IDistCurveEditor.h"
#include "Layout/Visibility.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FCurveEdInterface;
class FCurveEditorSharedData;
class FString;
class FText;
class FUICommandList;
class IMenu;
class SCurveEditorViewport;
class SHorizontalBox;
class STextComboBox;
class SWidget;
class UInterpCurveEdSetup;
class UObject;
struct FColor;
struct FCurveEdEntry;
template <typename FuncType> class TFunctionRef;

DECLARE_LOG_CATEGORY_EXTERN(LogCurveEd, Log, All);

/** The scope of a curve scaling operation */
namespace ECurveScaleScope
{
	enum Type
	{
		/** All curves in the current editor */
		All,

		/** The current curve */
		Current,

		/*  The current sub-curve */
		CurrentSub
	};
}

/*-----------------------------------------------------------------------------
   SDistributionCurveEditor
-----------------------------------------------------------------------------*/

class SDistributionCurveEditor : public IDistributionCurveEditor
{
public:
	SLATE_BEGIN_ARGS(SDistributionCurveEditor) 
		: _EdSetup(NULL)
		, _NotifyObject(NULL)
		, _CurveEdOptions(FCurveEdOptions())
		{}

		SLATE_ARGUMENT(UInterpCurveEdSetup*, EdSetup)
		SLATE_ARGUMENT(FCurveEdNotifyInterface*, NotifyObject)
		SLATE_ARGUMENT(FCurveEdOptions, CurveEdOptions)
	SLATE_END_ARGS()

	/** Constructor/Destructor */
	SDistributionCurveEditor();
	virtual ~SDistributionCurveEditor();

	/** SCompoundWidget functions */
	void Construct(const FArguments& InArgs);

	/** IDistributionCurveEditor interface */
	virtual void RefreshViewport() override;
	virtual void CurveChanged() override;
	virtual void SetCurveVisible(const UObject* InCurve, bool bShow) override;
	virtual void ClearAllVisibleCurves() override;
	virtual void SetCurveSelected(const UObject* InCurve, bool bSelected) override;
	virtual void ClearAllSelectedCurves() override;
	virtual void ScrollToFirstSelected() override;
	virtual void SetActiveTabToFirstSelected() override;
	virtual UInterpCurveEdSetup* GetEdSetup() override;
	virtual float GetStartIn() override;
	virtual float GetEndIn() override;
	virtual void SetPositionMarker(bool bEnabled, float InPosition, const FColor& InMarkerColor) override;
	virtual void SetEndMarker(bool bEnabled, float InEndPosition) override;
	virtual void SetRegionMarker(bool bEnabled, float InRegionStart, float InRegionEnd, const FColor& InRegionFillColor) override;
	virtual void SetInSnap(bool bEnabled, float SnapAmount, bool bInSnapToFrames) override;
	virtual void SetViewInterval(float StartIn, float EndIn) override;
	/** Fits the curve editor view horizontally to the curve data */
	virtual void FitViewHorizontally() override;
	/** Fits the curve editor view vertically to the curve data */
	virtual void FitViewVertically() override;

	/** Accessors */
	TSharedPtr<FCurveEditorSharedData> GetSharedData();

	/** Toolbar/menu command methods */
	void OnDeleteKeys();
	void OnFit();
	void OnFitToSelected();
	void OnFitHorizontally();
	void OnFitVertically();
	void OnSetTangentType(int32 NewType);

	/** Methods for opening context menus */
	void OpenLabelMenu();
	void OpenKeyMenu();
	void OpenGeneralMenu();
	void OpenCurveMenu();

	void CloseEntryPopup();
private:
	/** Creates the geometry mode controls */
	void CreateLayout(FCurveEdOptions CurveEdOptions);

	/** Query whether or not we're in small icon mode */
	EVisibility GetLargeIconVisibility() const;

	/** Builds the toolbar widget for the ParticleSystem editor */
	TSharedRef<SHorizontalBox> BuildToolBar();

	/**	Binds our UI commands to delegates */
	void BindCommands();

	/** Toolbar/menu command methods */
	void OnRemoveCurve();
	void OnRemoveAllCurves();
	void OnSetTime();
	void OnSetValue();
	void OnSetColor();
	void OnScaleTimes(ECurveScaleScope::Type Scope);
	void OnScaleValues(ECurveScaleScope::Type Scope);
	void OnScaleSingleCurveTimes();
	void OnScaleSingleCurveValues();
	void OnSetMode(int32 NewMode);
	bool IsModeChecked(int32 Mode) const;
	bool IsTangentTypeChecked(int32 Type) const;
	void OnFlattenTangents();
	void OnStraightenTangents();
	void OnShowAllTangents();
	bool IsShowAllTangentsChecked() const;
	void OnCreateTab();
	void OnDeleteTab();

	/** Methods for building context menus */
	TSharedRef<SWidget> BuildMenuWidgetLabel();
	TSharedRef<SWidget> BuildMenuWidgetKey();
	TSharedRef<SWidget> BuildMenuWidgetGeneral();
	TSharedRef<SWidget> BuildMenuWidgetCurve();

	/** Methods related to the tab combobox */
	void TabSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void SetTabSelection(TSharedPtr<FString> NewSelection, bool bUpdateWidget);

	/** On commit callbacks for various user input dialogs */
	void KeyTimeCommitted(const FText& CommentText, ETextCommit::Type CommitInfo);
	void KeyValueCommitted(const FText& CommentText, ETextCommit::Type CommitInfo);
	void ScaleTimeCommitted(const FText& CommentText, ETextCommit::Type CommitInfo, ECurveScaleScope::Type Scope);
	void ScaleValueCommitted(const FText& CommentText, ETextCommit::Type CommitInfo, ECurveScaleScope::Type Scope);
	void TabNameCommitted(const FText& CommentText, ETextCommit::Type CommitInfo);

	/** Helper function to handle undo/redo */
	bool NotifyPendingCurveChange(bool bSelectedOnly);

	/** Straightens or flattens all curve tangents */
	void ModifyTangents(bool bDoStraighten);

	/** Helper method to set selected tab */
	TSharedPtr<FString> GetSelectedTab() const;

	/** Helper function to iterate all selected curve keys if any are selected, otherwise all the keys in all the curves */
	void IterateKeys(TFunctionRef<void(int32, int32, FCurveEdEntry&, FCurveEdInterface&)> IteratorCallback);

private:
	/** A list commands to execute if a user presses the corresponding keybinding in the text box */
	TSharedRef<FUICommandList> UICommandList;

	/** Viewport */
	TSharedPtr<SCurveEditorViewport> Viewport;

	/** Toolbar */
	TSharedPtr<SHorizontalBox> Toolbar;

	/** Data and methods shared across multiple classes */
	TSharedPtr<FCurveEditorSharedData> SharedData;

	/** Reference to owner of the current popup */
	TWeakPtr<IMenu> EntryMenu;

	/** Tabs dropdown */
	TSharedPtr<STextComboBox> TabNamesComboBox;

	/** Names of the curve tabs */
	TArray< TSharedPtr<FString> > TabNames;

	/** Buffer amount used when fitting the viewport to the curve */
	const float FitMargin;

	/** Selected Tab to use */
	TSharedPtr<FString> SelectedTab;
};
