// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDistCurveEditor.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class UInterpCurveEdSetup;

extern const FName DistCurveEditorAppIdentifier;


/*-----------------------------------------------------------------------------
   IDistributionCurveEditorModule
-----------------------------------------------------------------------------*/

class IDistributionCurveEditorModule : public IModuleInterface
{
public:
	/**  */
	virtual TSharedRef<IDistributionCurveEditor> CreateCurveEditorWidget(UInterpCurveEdSetup* EdSetup, FCurveEdNotifyInterface* NotifyObject) = 0;
	virtual TSharedRef<IDistributionCurveEditor> CreateCurveEditorWidget(UInterpCurveEdSetup* EdSetup, FCurveEdNotifyInterface* NotifyObject, IDistributionCurveEditor::FCurveEdOptions Options) = 0;
};
