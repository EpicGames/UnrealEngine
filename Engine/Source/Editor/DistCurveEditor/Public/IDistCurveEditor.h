// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class UInterpCurveEdSetup;

/*-----------------------------------------------------------------------------
   FCurveEdNotifyInterface
-----------------------------------------------------------------------------*/

class FCurveEdNotifyInterface
{
public:
	virtual void PreEditCurve(TArray<UObject*> CurvesAboutToChange) {}
	virtual void PostEditCurve() {}

	virtual void MovedKey() {}

	virtual void DesireUndo() {}
	virtual void DesireRedo() {}

	/**
	 * Called by the Curve Editor when a Curve Label is clicked on
	 *
	 * @param	CurveObject	The curve object whose label was clicked on
	 */
	virtual void OnCurveLabelClicked(UObject* CurveObject) {}
};


/*-----------------------------------------------------------------------------
   IDistributionCurveEditor
-----------------------------------------------------------------------------*/

class IDistributionCurveEditor : public SCompoundWidget
{
public:
	/** Refreshes the viewport */
	virtual void RefreshViewport() = 0;

	/** Clears selected keys and updates the viewport */
	virtual void CurveChanged() = 0;

	/** Shows a curve */
	virtual void SetCurveVisible(const UObject* InCurve, bool bShow) = 0;

	/** Hides all curves */
	virtual void ClearAllVisibleCurves() = 0;

	/** Selects a curve */
	virtual void SetCurveSelected(const UObject* InCurve, bool bSelected) = 0;

	/** Deselects all curves */
	virtual void ClearAllSelectedCurves() = 0;

	/** Scrolls the curve labels to the first selected curve */
	virtual void ScrollToFirstSelected() = 0;

	/** Finds/activates the tab containing the first selected curve */
	virtual void SetActiveTabToFirstSelected() = 0;

	/** Accessors */
	virtual UInterpCurveEdSetup* GetEdSetup() = 0;
	virtual float GetStartIn() = 0;
	virtual float GetEndIn() = 0;
	virtual void SetPositionMarker(bool bEnabled, float InPosition, const FColor& InMarkerColor) = 0;
	virtual void SetEndMarker(bool bEnabled, float InEndPosition) = 0;
	virtual void SetRegionMarker(bool bEnabled, float InRegionStart, float InRegionEnd, const FColor& InRegionFillColor) = 0;
	virtual void SetInSnap(bool bEnabled, float SnapAmount, bool bInSnapToFrames) = 0;
	virtual void SetViewInterval(float StartIn, float EndIn) = 0;

	/** Fit Accessors */
	virtual void FitViewHorizontally() = 0;
	virtual void FitViewVertically() = 0;

	/** Additional Options */
	struct FCurveEdOptions
	{
		bool bAlwaysShowScrollbar;	
		FCurveEdOptions()
		:bAlwaysShowScrollbar(false)
		{}
	};
};
